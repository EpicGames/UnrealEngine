// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ControllerComponent.h"
#include "Components/GameFrameworkComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControllerComponent)

UControllerComponent::UControllerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UControllerComponent::IsLocalController() const
{
	return GetControllerChecked<AController>()->IsLocalController();
}

void UControllerComponent::GetPlayerViewPoint(FVector& Location, FRotator& Rotation) const
{
	GetControllerChecked<AController>()->GetPlayerViewPoint(Location, Rotation);
}
