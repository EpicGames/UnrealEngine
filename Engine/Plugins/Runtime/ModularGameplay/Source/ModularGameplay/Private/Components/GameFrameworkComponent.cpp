// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/GameFrameworkComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFrameworkComponent)

UGameFrameworkComponent::UGameFrameworkComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = false;
}

bool UGameFrameworkComponent::HasAuthority() const
{
	AActor* Owner = GetOwner();
	check(Owner);
	return Owner->HasAuthority();
}

FTimerManager& UGameFrameworkComponent::GetWorldTimerManager() const
{
	AActor* Owner = GetOwner();
	check(Owner);
	return Owner->GetWorldTimerManager();
}
