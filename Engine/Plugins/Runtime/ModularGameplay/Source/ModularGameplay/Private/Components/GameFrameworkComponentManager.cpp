// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/GameFrameworkComponentManager.h"
#include "Components/GameFrameworkComponentDelegates.h"
#include "Engine/GameInstance.h"
#include "EngineUtils.h"
#include "ModularGameplayLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFrameworkComponentManager)

#if !UE_BUILD_SHIPPING
#include "UObject/UObjectIterator.h"
static FAutoConsoleCommand CVarDumpGameFrameworkComponentManagers(
	TEXT("ModularGameplay.DumpGameFrameworkComponentManagers"),
	TEXT("Lists all active component requests, all receiver actors, and all instanced components on all game framework component managers."),
	FConsoleCommandDelegate::CreateStatic(UGameFrameworkComponentManager::DumpGameFrameworkComponentManagers));
#endif // !UE_BUILD_SHIPPING

FComponentRequestHandle::~FComponentRequestHandle()
{
	UGameFrameworkComponentManager* LocalManager = OwningManager.Get();
	if (LocalManager)
	{
		if (ComponentClass.Get())
		{
			LocalManager->RemoveComponentRequest(ReceiverClass, ComponentClass);
		}
		if (ExtensionHandle.IsValid())
		{
			LocalManager->RemoveExtensionHandler(ReceiverClass, ExtensionHandle);
		}
	}
}

bool FComponentRequestHandle::IsValid() const
{
	return OwningManager.IsValid();
}

FName UGameFrameworkComponentManager::NAME_ReceiverAdded = FName("ReceiverAdded");
FName UGameFrameworkComponentManager::NAME_ReceiverRemoved = FName("ReceiverRemoved");
FName UGameFrameworkComponentManager::NAME_ExtensionAdded = FName("ExtensionAdded");
FName UGameFrameworkComponentManager::NAME_ExtensionRemoved = FName("ExtensionRemoved");
FName UGameFrameworkComponentManager::NAME_GameActorReady = FName("GameActorReady");

void UGameFrameworkComponentManager::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	if (UGameFrameworkComponentManager* GFCM = Cast<UGameFrameworkComponentManager>(InThis))
	{
		for (auto MapIt = GFCM->ReceiverClassToComponentClassMap.CreateIterator(); MapIt; ++MapIt)
		{
			for (auto& ValueElement : MapIt.Value())
			{
				Collector.AddReferencedObject(ValueElement.Class);
			}
		}
	}
}

#if !UE_BUILD_SHIPPING
void UGameFrameworkComponentManager::DumpGameFrameworkComponentManagers()
{
	UE_LOGF(LogModularGameplay, Display, "Dumping GameFrameworkComponentManagers...");
	for (TObjectIterator<UGameFrameworkComponentManager> ManagerIt; ManagerIt; ++ManagerIt)
	{
		if (UGameFrameworkComponentManager* Manager = *ManagerIt)
		{
			UE_LOGF(LogModularGameplay, Display, "  Manager: %ls", *GetPathNameSafe(Manager));

#if WITH_EDITOR
			UE_LOGF(LogModularGameplay, Display, "    Receivers... (Num:%d)", Manager->AllReceivers.Num());
			for (auto SetIt = Manager->AllReceivers.CreateConstIterator(); SetIt; ++SetIt)
			{
				UE_LOGF(LogModularGameplay, Display, "      ReceiverInstance: %ls", *GetPathNameSafe(SetIt->ResolveObjectPtr()));
			}
#endif // WITH_EDITOR

			UE_LOGF(LogModularGameplay, Display, "    Components... (Num:%d)", Manager->ComponentClassToComponentInstanceMap.Num());
			for (auto MapIt = Manager->ComponentClassToComponentInstanceMap.CreateConstIterator(); MapIt; ++MapIt)
			{
				UE_LOGF(LogModularGameplay, Display, "      ComponentClass: %ls (Num:%d)", *GetPathNameSafe(MapIt.Key()), MapIt.Value().Num());
				for (const FObjectKey& ComponentInstance : MapIt.Value())
				{
					UE_LOGF(LogModularGameplay, Display, "        ComponentInstance: %ls", *GetPathNameSafe(ComponentInstance.ResolveObjectPtr()));
				}
			}
			UE_LOGF(LogModularGameplay, Display, "    Requests... (Num:%d)", Manager->ReceiverClassToComponentClassMap.Num());
			for (auto MapIt = Manager->ReceiverClassToComponentClassMap.CreateConstIterator(); MapIt; ++MapIt)
			{
				UE_LOGF(LogModularGameplay, Display, "      RequestReceiverClass: %ls (Num:%d)", *MapIt.Key().ToDebugString(), MapIt.Value().Num());
				for (const FComponentRequestInfo& ReceiverInfo : MapIt.Value())
				{
					UE_LOGF(LogModularGameplay, Display, "        RequestComponentClass: %ls	AdditionFlags: %u", *GetPathNameSafe(ReceiverInfo.Class), EnumToUnderlyingType(ReceiverInfo.AdditionFlags));
				}
			}
		}
	}
}
#endif // !UE_BUILD_SHIPPING

void UGameFrameworkComponentManager::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	CurrentStateChange = INDEX_NONE;

	FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.AddUObject(this, &UGameFrameworkComponentManager::PruneUnboundDelegates);

#if WITH_EDITORONLY_DATA
	FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UGameFrameworkComponentManager::PostGC);
#endif
}

void UGameFrameworkComponentManager::Deinitialize() 
{
	Super::Deinitialize();

	FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.RemoveAll(this);

#if WITH_EDITORONLY_DATA
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);
#endif
}

#if WITH_EDITORONLY_DATA
void UGameFrameworkComponentManager::PostGC()
{
	// Clear invalid receivers. 
	for (auto It = AllReceivers.CreateIterator(); It; ++It)
	{
		if (It->ResolveObjectPtr() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
}
#endif

UGameFrameworkComponentManager* UGameFrameworkComponentManager::GetForActor(const AActor* Actor, bool bOnlyGameWorlds)
{
	if (Actor)
	{
		if (UWorld* ReceiverWorld = Actor->GetWorld())
		{
			if (bOnlyGameWorlds && (!ReceiverWorld->IsGameWorld() || ReceiverWorld->IsPreviewWorld()))
			{
				return nullptr;
			}

			return UGameInstance::GetSubsystem<UGameFrameworkComponentManager>(ReceiverWorld->GetGameInstance());
		}
	}

	return nullptr;
}

void UGameFrameworkComponentManager::AddReceiver(AActor* Receiver, bool bAddOnlyInGameWorlds)
{
	if (Receiver != nullptr)
	{
		if (bAddOnlyInGameWorlds)
		{
			UWorld* ReceiverWorld = Receiver->GetWorld();
			if ((ReceiverWorld == nullptr) || !ReceiverWorld->IsGameWorld() || ReceiverWorld->IsPreviewWorld())
			{
				return;
			}
		}

		AddReceiverInternal(Receiver);
	}
}

void UGameFrameworkComponentManager::AddReceiverInternal(AActor* Receiver)
{
	checkSlow(Receiver);
	
#if WITH_EDITOR
	AllReceivers.Add(FObjectKey(Receiver));
#endif
	
	for (UClass* Class = Receiver->GetClass(); Class && Class != AActor::StaticClass(); Class = Class->GetSuperClass())
	{
		FComponentRequestReceiverClassPath ReceiverClassPath(Class);
		if (auto* RequestInfoSet = ReceiverClassToComponentClassMap.Find(ReceiverClassPath))
		{
			for (const FComponentRequestInfo& SetInfo : *RequestInfoSet)
			{
				if (SetInfo.Class)
				{
					CreateComponentOnInstance(Receiver, SetInfo.Class, SetInfo.AdditionFlags);
				}
			}
		}

		if (FExtensionHandlerEvent* HandlerEvent = ReceiverClassToEventMap.Find(ReceiverClassPath))
		{
			// Copy the execution list so it isn't invalidated
			FExtensionHandlerEvent HandlerCopy = *HandlerEvent;
			for (const TPair<FDelegateHandle, TSharedRef<FExtensionHandlerRegisteredDelegate>>& Pair : HandlerCopy)
			{
				Pair.Value->Execute(Receiver, NAME_ReceiverAdded);
			}
		}
	}
}

void UGameFrameworkComponentManager::RemoveReceiver(AActor* Receiver)
{
	if (Receiver != nullptr)
	{
		RemoveReceiverInternal(Receiver);
	}
}

void UGameFrameworkComponentManager::RemoveReceiverInternal(AActor* Receiver)
{
	checkSlow(Receiver);
	
#if WITH_EDITOR
	ensureMsgf(AllReceivers.Remove(Receiver) > 0, TEXT("Called RemoveReceiver without first calling AddReceiver. Actor:%s"), *GetPathNameSafe(Receiver));
#endif
	
	TInlineComponentArray<UActorComponent*> ComponentsToDestroy;
	for (UActorComponent* Component : Receiver->GetComponents())
	{
		if (UActorComponent* GFC = Cast<UActorComponent>(Component))
		{
			UClass* ComponentClass = GFC->GetClass();
			TSet<FObjectKey>* ComponentInstances = ComponentClassToComponentInstanceMap.Find(ComponentClass);
			if (ComponentInstances)
			{
				if (ComponentInstances->Contains(GFC))
				{
					ComponentsToDestroy.Add(GFC);
				}
			}
		}
	}

	for (UActorComponent* Component : ComponentsToDestroy)
	{
		DestroyInstancedComponent(Component);
	}

	SendExtensionEventInternal(Receiver, NAME_ReceiverRemoved);
}

TSharedPtr<FComponentRequestHandle> UGameFrameworkComponentManager::AddComponentRequest(const TSoftClassPtr<AActor>& ReceiverClass, TSubclassOf<UActorComponent> ComponentClass, const EGameFrameworkAddComponentFlags AdditionFlags)
{
	// You must have a receiver and component class. The receiver cannot be AActor, that is too broad and would be bad for performance.
	if (!ensure(!ReceiverClass.IsNull()) || !ensure(ComponentClass) || !ensure(ReceiverClass.ToString() != TEXT("/Script/Engine.Actor")))
	{
		return nullptr;
	}

	FComponentRequestReceiverClassPath ReceiverClassPath(ReceiverClass);
	UClass* ComponentClassPtr = ComponentClass.Get();

	FComponentRequest NewRequest;
	NewRequest.ReceiverClassPath = ReceiverClassPath;
	NewRequest.ComponentClass = ComponentClassPtr;
	
	// Add a request if there is not an already existing one. Note that it will only uses the receiver and component class to check for uniqueness, not the addition flags.
	int32& RequestCount = RequestTrackingMap.FindOrAdd(NewRequest);
	RequestCount++;

	if (RequestCount == 1)
	{
		EGameFrameworkAddComponentResult Result = EGameFrameworkAddComponentResult::Failed;
		auto& RequestInfoSet = ReceiverClassToComponentClassMap.FindOrAdd(ReceiverClassPath);
		RequestInfoSet.Add({ ComponentClassPtr, AdditionFlags } );
		if (UClass* ReceiverClassPtr = ReceiverClass.Get())
		{
			UGameInstance* LocalGameInstance = GetGameInstance();
			if (ensure(LocalGameInstance))
			{
				UWorld* LocalWorld = LocalGameInstance->GetWorld();
				if (ensure(LocalWorld))
				{
					for (TActorIterator<AActor> ActorIt(LocalWorld, ReceiverClassPtr); ActorIt; ++ActorIt)
					{
						if (ActorIt->IsActorInitialized())
						{
#if WITH_EDITOR
							if (!ReceiverClassPtr->HasAllClassFlags(CLASS_Abstract))
							{
								ensureMsgf(AllReceivers.Contains(*ActorIt), TEXT("You may not add a component request for an actor class that does not call AddReceiver/RemoveReceiver in code! Class:%s"), *GetPathNameSafe(ReceiverClassPtr));
							}
#endif
							Result = CreateComponentOnInstance(*ActorIt, ComponentClass, AdditionFlags);
						}
					}
				}
			}
		}
		else
		{
			// Actor class is not in memory, there will be no actor instances
		}
	}

	return MakeShared<FComponentRequestHandle>(this, ReceiverClass, ComponentClass);
}

void UGameFrameworkComponentManager::RemoveComponentRequest(const TSoftClassPtr<AActor>& ReceiverClass, TSubclassOf<UActorComponent> ComponentClass)
{
	FComponentRequestReceiverClassPath ReceiverClassPath(ReceiverClass);
	UClass* ComponentClassPtr = ComponentClass.Get();

	FComponentRequest NewRequest;
	NewRequest.ReceiverClassPath = ReceiverClassPath;
	NewRequest.ComponentClass = ComponentClassPtr;
	int32& RequestCount = RequestTrackingMap.FindChecked(NewRequest);
	check(RequestCount > 0);
	RequestCount--;

	if (RequestCount == 0)
	{
		if (TSet<FComponentRequestInfo>* ReceiverSetInfo = ReceiverClassToComponentClassMap.Find(ReceiverClassPath))
		{
			ReceiverSetInfo->Remove(ComponentClassPtr);
			if (ReceiverSetInfo->Num() == 0)
			{
				ReceiverClassToComponentClassMap.Remove(ReceiverClassPath);
			}
		}

		if (UClass* ReceiverClassPtr = ReceiverClass.Get())
		{
			if (TSet<FObjectKey>* ComponentInstances = ComponentClassToComponentInstanceMap.Find(ComponentClassPtr))
			{
				TArray<UActorComponent*> ComponentsToDestroy;
				for (const FObjectKey& InstanceKey : *ComponentInstances)
				{
					UActorComponent* Comp = Cast<UActorComponent>(InstanceKey.ResolveObjectPtr());
					if (Comp)
					{
						AActor* OwnerActor = Comp->GetOwner();
						if (OwnerActor && OwnerActor->IsA(ReceiverClassPtr))
						{
							ComponentsToDestroy.Add(Comp);
						}
					}
				}

				for (UActorComponent* Component : ComponentsToDestroy)
				{
					DestroyInstancedComponent(Component);
				}
			}
		}
	}
}

TSharedPtr<FComponentRequestHandle> UGameFrameworkComponentManager::AddExtensionHandler(const TSoftClassPtr<AActor>& ReceiverClass, FExtensionHandlerDelegate ExtensionHandler)
{
	// You must have a target and bound handler. The target cannot be AActor, that is too broad and would be bad for performance.
	if (!ensure(!ReceiverClass.IsNull()) || !ensure(ExtensionHandler.IsBound()) || !ensure(ReceiverClass.ToString() != TEXT("/Script/Engine.Actor")))
	{
		return nullptr;
	}

	FComponentRequestReceiverClassPath ReceiverClassPath(ReceiverClass);
	FExtensionHandlerEvent& HandlerEvent = ReceiverClassToEventMap.FindOrAdd(ReceiverClassPath);

	// This is a fake multicast delegate using a map, we store it in shared memory to avoid things being reallocated during execution
	// This avoids copying the delegate itself which is often expensive
	FDelegateHandle DelegateHandle(FDelegateHandle::EGenerateNewHandleType::GenerateNewHandle);
	TSharedRef<FExtensionHandlerRegisteredDelegate> RegisteredHandler = MakeShared<FExtensionHandlerRegisteredDelegate>(MoveTemp(ExtensionHandler));
	HandlerEvent.Add(DelegateHandle, RegisteredHandler);

	if (UClass* ReceiverClassPtr = ReceiverClass.Get())
	{
		UGameInstance* LocalGameInstance = GetGameInstance();
		if (ensure(LocalGameInstance))
		{
			UWorld* LocalWorld = LocalGameInstance->GetWorld();
			if (ensure(LocalWorld))
			{
				for (TActorIterator<AActor> ActorIt(LocalWorld, ReceiverClassPtr); ActorIt; ++ActorIt)
				{
					if (ActorIt->IsActorInitialized())
					{
						RegisteredHandler->Execute(*ActorIt, NAME_ExtensionAdded);
					}
				}
			}
		}
	}
	else
	{
		// Actor class is not in memory, there will be no actor instances
	}

	return MakeShared<FComponentRequestHandle>(this, ReceiverClass, DelegateHandle);
}

void UGameFrameworkComponentManager::RemoveExtensionHandler(const TSoftClassPtr<AActor>& ReceiverClass, FDelegateHandle DelegateHandle)
{
	FComponentRequestReceiverClassPath ReceiverClassPath(ReceiverClass);

	if (FExtensionHandlerEvent* HandlerEvent = ReceiverClassToEventMap.Find(ReceiverClassPath))
	{
		TSharedRef<FExtensionHandlerRegisteredDelegate>* HandlerDelegate = HandlerEvent->Find(DelegateHandle);
		if (ensure(HandlerDelegate))
		{
			TSharedRef<FExtensionHandlerRegisteredDelegate> HandlerRef = *HandlerDelegate;
			// Call it once on unregister, but skip during engine shutdown
			if (UClass* ReceiverClassPtr = ReceiverClass.Get())
			{
				UGameInstance* LocalGameInstance = GetGameInstance();
				if (!IsEngineExitRequested() && ensure(LocalGameInstance))
				{
					UWorld* LocalWorld = LocalGameInstance->GetWorld();
					ensure(GIsEditor || (LocalWorld != nullptr));
					if (LocalWorld)
					{
						for (TActorIterator<AActor> ActorIt(LocalWorld, ReceiverClassPtr); ActorIt; ++ActorIt)
						{
							if (ActorIt->IsActorInitialized())
							{
								HandlerRef->Execute(*ActorIt, NAME_ExtensionRemoved);
							}
						}
					}
				}
			}
			else
			{
				// Actor class is not in memory, there will be no actor instances
			}

			// This stops it from executing further up the stack
			HandlerRef->bRemoved = true;
			HandlerEvent->Remove(DelegateHandle);

			if (HandlerEvent->IsEmpty())
			{
				ReceiverClassToEventMap.Remove(ReceiverClassPath);
			}
		}
	}
}

void UGameFrameworkComponentManager::SendExtensionEvent(AActor* Receiver, FName EventName, bool bOnlyInGameWorlds)
{
	if (Receiver != nullptr)
	{
		if (bOnlyInGameWorlds)
		{
			UWorld* ReceiverWorld = Receiver->GetWorld();
			if ((ReceiverWorld == nullptr) || !ReceiverWorld->IsGameWorld() || ReceiverWorld->IsPreviewWorld())
			{
				return;
			}
		}

		SendExtensionEventInternal(Receiver, EventName);
	}
}

void UGameFrameworkComponentManager::SendExtensionEventInternal(AActor* Receiver, const FName& EventName)
{
	for (UClass* Class = Receiver->GetClass(); Class && Class != AActor::StaticClass(); Class = Class->GetSuperClass())
	{
		FComponentRequestReceiverClassPath ReceiverClassPath(Class);
		if (FExtensionHandlerEvent* HandlerEvent = ReceiverClassToEventMap.Find(ReceiverClassPath))
		{
			// Copy the execution list so it isn't invalidated
			FExtensionHandlerEvent HandlerCopy = *HandlerEvent;
			for (const TPair<FDelegateHandle, TSharedRef<FExtensionHandlerRegisteredDelegate>>& Pair : HandlerCopy)
			{
				Pair.Value->Execute(Receiver, EventName);
			}
		}
	}
}

EGameFrameworkAddComponentResult UGameFrameworkComponentManager::CreateComponentOnInstance(AActor* ActorInstance, TSubclassOf<UActorComponent> ComponentClass, const EGameFrameworkAddComponentFlags AdditionFlags)
{
	check(ActorInstance);
	check(ComponentClass);

	if (!ComponentClass->GetDefaultObject<UActorComponent>()->GetIsReplicated() || ActorInstance->GetLocalRole() == ROLE_Authority)
	{
		// If AddUnique is set, it will be added only if no component on ActorInstance is child (or same class) of ComponentClass
		const bool bAddUnique = EnumHasAnyFlags(AdditionFlags, EGameFrameworkAddComponentFlags::AddUnique);
		if (bAddUnique)
		{
			if (ActorInstance->GetComponentByClass(ComponentClass))
			{
				return EGameFrameworkAddComponentResult::Failed;
			}
		}

		// If AddIfNotChild is set,it will be added only if ComponentClass is not a child of an existing component on ActorInstance
		const bool bAddIfNotChild = EnumHasAnyFlags(AdditionFlags, EGameFrameworkAddComponentFlags::AddIfNotChild);
		if (bAddIfNotChild)
		{
			const TSet<UActorComponent*>& Components = ActorInstance->GetComponents();
			for (const UActorComponent* ActorComp : Components)
			{
				if (ComponentClass->IsChildOf(ActorComp->GetClass()))
				{
					return EGameFrameworkAddComponentResult::Failed;
				}
			}
		}

		// If UseAutoGeneratedName is set, it will generate a new name and not re-use the class name directly (which can lead to component recycling)
		const bool bUseAutoGeneratedName = EnumHasAnyFlags(AdditionFlags, EGameFrameworkAddComponentFlags::UseAutoGeneratedName);
		const FName NewComponentName = bUseAutoGeneratedName ? NAME_None : ComponentClass->GetFName();

		if (!bUseAutoGeneratedName)
		{
			// If we specify the name, this could replace an already existing object. 
			if (UObject* ExistingObject = StaticFindObjectFast(nullptr, ActorInstance, NewComponentName))
			{
				if (ComponentClass->IsChildOf(ExistingObject->GetClass()))
				{
					// If they are related classes this will work but probably indicates an improper game feature action setup
					// If it is intended to add a component from two different places, the creation flags like AddUnique can be used to only add if it does not yet exist
					UE_LOGF(LogModularGameplay, Warning, "CreateComponentOnInstance is replacing existing object %ls with class %ls using flags 0x%08x, check for duplicate game feature actions.",
						*ExistingObject->GetFullName(nullptr, EObjectFullNameFlags::IncludeClassPackage),
						*ComponentClass->GetPathName(), (int32)AdditionFlags);
				}
				else
				{
					// This will fatally crash inside AllocateObject, so error and cancel instead. Log and ensure so each case will write to the log
					UE_LOGF(LogModularGameplay, Error, "CreateComponentOnInstance cannot replace object %ls with incompatible class %ls using flags 0x%08x!",
						*ExistingObject->GetFullName(nullptr, EObjectFullNameFlags::IncludeClassPackage),
						*ComponentClass->GetPathName(), (int32)AdditionFlags);

					ensureMsgf(false, TEXT("CreateComponentOnInstance cannot replace object with incompatible class, check log for details"));

					return EGameFrameworkAddComponentResult::Failed;
				}
			}
		}

		UActorComponent* NewComp = NewObject<UActorComponent>(ActorInstance, ComponentClass, NewComponentName);
		TSet<FObjectKey>& ComponentInstances = ComponentClassToComponentInstanceMap.FindOrAdd(*ComponentClass);
		ComponentInstances.Add(NewComp);

		if (USceneComponent* NewSceneComp = Cast<USceneComponent>(NewComp))
		{
			NewSceneComp->SetupAttachment(ActorInstance->GetRootComponent());
		}

		NewComp->RegisterComponent();

		return EGameFrameworkAddComponentResult::Success;
	}

	return EGameFrameworkAddComponentResult::Failed;
}

void UGameFrameworkComponentManager::DestroyInstancedComponent(UActorComponent* Component)
{
	check(Component);

	UClass* ComponentClass = Component->GetClass();
	if (TSet<FObjectKey>* ComponentInstances = ComponentClassToComponentInstanceMap.Find(ComponentClass))
	{
		ComponentInstances->Remove(Component);
		if (ComponentInstances->Num() == 0)
		{
			ComponentClassToComponentInstanceMap.Remove(ComponentClass);
		}
	}
	Component->DestroyComponent();
	Component->SetFlags(RF_Transient);
}

void UGameFrameworkComponentManager::AddGameFrameworkComponentReceiver(AActor* Receiver, bool bAddOnlyInGameWorlds)
{
	if (UGameFrameworkComponentManager* GFCM = GetForActor(Receiver, bAddOnlyInGameWorlds))
	{
		GFCM->AddReceiverInternal(Receiver);
	}
}

void UGameFrameworkComponentManager::RemoveGameFrameworkComponentReceiver(AActor* Receiver)
{
	if (UGameFrameworkComponentManager* GFCM = GetForActor(Receiver, false))
	{
		GFCM->RemoveReceiverInternal(Receiver);
	}
}

void UGameFrameworkComponentManager::SendGameFrameworkComponentExtensionEvent(AActor* Receiver, const FName& EventName, bool bOnlyInGameWorlds)
{
	if (UGameFrameworkComponentManager* GFCM = GetForActor(Receiver, bOnlyInGameWorlds))
	{
		// bOnlyInGameWorlds and the null check is handled by GetForActor, so skip the check in SendExtensionEvent
		GFCM->SendExtensionEventInternal(Receiver, EventName);
	}
}

UGameFrameworkComponentManager::FActorFeatureRegisteredDelegate::FActorFeatureRegisteredDelegate(FActorInitStateChangedDelegate&& InDelegate, FName InFeatureName, FGameplayTag InInitState)
	: Delegate(InDelegate)
	, DelegateHandle(FDelegateHandle::EGenerateNewHandleType::GenerateNewHandle)
	, RequiredFeatureName(InFeatureName)
	, RequiredInitState(InInitState)
	, bRemoved(false)
{

}

UGameFrameworkComponentManager::FActorFeatureRegisteredDelegate::FActorFeatureRegisteredDelegate(FActorInitStateChangedBPDelegate&& InDelegate, FName InFeatureName, FGameplayTag InInitState)
	: BPDelegate(InDelegate)
	, DelegateHandle(FDelegateHandle::EGenerateNewHandleType::GenerateNewHandle)
	, RequiredFeatureName(InFeatureName)
	, RequiredInitState(InInitState)
	, bRemoved(false)
{

}

void UGameFrameworkComponentManager::FActorFeatureRegisteredDelegate::Execute(AActor* OwningActor, FName FeatureName, UObject* Implementer, FGameplayTag FeatureState)
{
	if (bRemoved)
	{
		return;
	}

	FActorInitStateChangedParams Params(OwningActor, FeatureName, Implementer, FeatureState);
	if (Delegate.IsBound())
	{
		ensure(!BPDelegate.IsBound());

		Delegate.Execute(Params);
	}
	else if (BPDelegate.IsBound())
	{
		BPDelegate.Execute(Params);
	}
}

UGameFrameworkComponentManager::FActorFeatureData::~FActorFeatureData()
{
	for (TSharedRef<FActorFeatureRegisteredDelegate>& RegisteredDelegate : RegisteredDelegates)
	{
		// This ensures it will not execute if cleared during delegate iteration
		RegisteredDelegate->bRemoved = true;
	}
}

void UGameFrameworkComponentManager::RegisterInitState(FGameplayTag NewState, bool bAddBefore, FGameplayTag ExistingState)
{
	int32 FoundIndex;

	// TODO ensure or ignore for duplicates?
	if (InitStateOrder.Contains(NewState))
	{
		return;
	}

	if (!ExistingState.IsValid())
	{
		if (bAddBefore)
		{
			InitStateOrder.Insert(NewState, 0);
		}
		else
		{
			InitStateOrder.Add(NewState);
		}
	}

	else if (ensureMsgf(InitStateOrder.Find(ExistingState, FoundIndex), TEXT("State %s not found in existing state list!"), *ExistingState.ToString()))
	{
		if (!bAddBefore)
		{
			FoundIndex++;
		}

		InitStateOrder.Insert(NewState, FoundIndex);
	}
}

bool UGameFrameworkComponentManager::IsInitStateAfterOrEqual(FGameplayTag FeatureState, FGameplayTag RelativeState) const
{
	// In theory we could use a tree instead of a simple array, but if two states are related to each other they should be registered relative to each other
	// And if the two states are completely unrelated, the order doesn't matter because it will never be queried

	if (FeatureState == RelativeState)
	{
		return true;
	}

	bool bFoundSecond = false;
	for (int32 i = 0; i < InitStateOrder.Num(); i++)
	{
		if (InitStateOrder[i] == RelativeState)
		{
			bFoundSecond = true;
		}
		else if (InitStateOrder[i] == FeatureState)
		{
			return bFoundSecond;
		}
	}

	return false;
}

FGameplayTag UGameFrameworkComponentManager::GetInitStateForFeature(AActor* Actor, FName FeatureName) const
{
	const FActorFeatureState* FoundState = FindFeatureStateStruct(ActorFeatureMap.Find(FObjectKey(Actor)), FeatureName, FGameplayTag());

	if (FoundState)
	{
		return FoundState->CurrentState;
	}

	return FGameplayTag();
}

bool UGameFrameworkComponentManager::HasFeatureReachedInitState(AActor* Actor, FName FeatureName, FGameplayTag FeatureState) const
{
	FGameplayTag FoundState = GetInitStateForFeature(Actor, FeatureName);
	return IsInitStateAfterOrEqual(FoundState, FeatureState);
}

UObject* UGameFrameworkComponentManager::GetImplementerForFeature(AActor* Actor, FName FeatureName, FGameplayTag RequiredState) const
{
	const FActorFeatureState* FoundState = FindFeatureStateStruct(ActorFeatureMap.Find(FObjectKey(Actor)), FeatureName, RequiredState);

	if (FoundState)
	{
		return FoundState->Implementer.Get();
	}

	return nullptr;
}

void UGameFrameworkComponentManager::GetAllFeatureImplementers(TArray<UObject*>& OutImplementers, AActor* Actor, FGameplayTag RequiredState, FName ExcludingFeature /*= NAME_None*/) const
{
	const FActorFeatureData* FoundStruct = ActorFeatureMap.Find(FObjectKey(Actor));

	if (FoundStruct)
	{
		for (const FActorFeatureState& State : FoundStruct->RegisteredStates)
		{
			if (State.FeatureName != ExcludingFeature)
			{
				if (!RequiredState.IsValid() || IsInitStateAfterOrEqual(State.CurrentState, RequiredState))
				{
					if (UObject* Implementer = State.Implementer.Get())
					{
						OutImplementers.Add(Implementer);
					}
				}
			}
		}
	}
}

bool UGameFrameworkComponentManager::HaveAllFeaturesReachedInitState(AActor* Actor, FGameplayTag RequiredState, FName ExcludingFeature) const
{
	const FActorFeatureData* FoundStruct = ActorFeatureMap.Find(FObjectKey(Actor));

	if (FoundStruct)
	{
		for (const FActorFeatureState& State : FoundStruct->RegisteredStates)
		{
			if (State.FeatureName != ExcludingFeature)
			{
				if (!IsInitStateAfterOrEqual(State.CurrentState, RequiredState))
				{
					return false;
				}
			}
		}

		// TODO do we want at least one feature to be valid?
		return true;
	}

	return false;
}

bool UGameFrameworkComponentManager::ChangeFeatureInitState(AActor* Actor, FName FeatureName, UObject* Implementer, FGameplayTag FeatureState)
{
	if (Actor == nullptr || FeatureName.IsNone() || !FeatureState.IsValid())
	{
		// TODO Ensure?
		return false;
	}

	FActorFeatureData& ActorStruct = FindOrAddActorData(Actor);

	FActorFeatureState* FoundState = nullptr;
	for (FActorFeatureState& State : ActorStruct.RegisteredStates)
	{
		if (State.FeatureName == FeatureName)
		{
			// TODO what if it's already in the desired state?
			FoundState = &State;
		}
	}

	if (!FoundState)
	{
		FoundState = &ActorStruct.RegisteredStates.Emplace_GetRef(FeatureName);
	}

	FoundState->CurrentState = FeatureState;
	FoundState->Implementer = Implementer;

	ProcessFeatureStateChange(Actor, FoundState);

	return true;
}

bool UGameFrameworkComponentManager::RegisterFeatureImplementer(AActor* Actor, FName FeatureName, UObject* Implementer)
{
	if (Actor == nullptr || FeatureName.IsNone())
	{
		// TODO Ensure?
		return false;
	}

	FActorFeatureData& ActorStruct = FindOrAddActorData(Actor);

	FActorFeatureState* FoundState = nullptr;
	for (FActorFeatureState& State : ActorStruct.RegisteredStates)
	{
		if (State.FeatureName == FeatureName)
		{
			// TODO what if it's already in the desired state?
			FoundState = &State;
		}
	}

	if (!FoundState)
	{
		FoundState = &ActorStruct.RegisteredStates.Emplace_GetRef(FeatureName);
	}

	FoundState->Implementer = Implementer;
	return true;
}

void UGameFrameworkComponentManager::RemoveActorFeatureData(AActor* Actor)
{
	// TODO when should we clear up now-invalid actor keys?
	ActorFeatureMap.Remove(FObjectKey(Actor));
}

void UGameFrameworkComponentManager::RemoveFeatureImplementer(AActor* Actor, UObject* Implementer)
{
	if (Actor == nullptr || Implementer == nullptr)
	{
		return;
	}
	TWeakObjectPtr<UObject> WeakToRemove(Implementer);

	FActorFeatureData* FoundStruct = ActorFeatureMap.Find(FObjectKey(Actor));

	if (FoundStruct)
	{
		for (int32 i = FoundStruct->RegisteredStates.Num() - 1; i >= 0; i--)
		{
			// Clear if it matches or is stale
			UObject* ResolvedObject = FoundStruct->RegisteredStates[i].Implementer.Get();

			if (ResolvedObject == Implementer || (!ResolvedObject && !FoundStruct->RegisteredStates[i].Implementer.IsExplicitlyNull()))
			{
				FoundStruct->RegisteredStates.RemoveAt(i);
			}
		}
	}
}

FDelegateHandle UGameFrameworkComponentManager::RegisterAndCallForActorInitState(AActor* Actor, FName FeatureName, FGameplayTag RequiredState, FActorInitStateChangedDelegate Delegate, bool bCallImmediately)
{
	if (ensure(Actor && Delegate.IsBound()))
	{
		// We often register delegates before registering states
		FActorFeatureData& ActorStruct = FindOrAddActorData(Actor);

		TSharedRef<FActorFeatureRegisteredDelegate> RegisteredDelegate = MakeShared<FActorFeatureRegisteredDelegate>(MoveTemp(Delegate), FeatureName, RequiredState);
		ActorStruct.RegisteredDelegates.Add(RegisteredDelegate);

		if (bCallImmediately)
		{
			// The shared ref keeps the delegate alive in case it gets unregistered
			CallDelegateForMatchingFeatures(Actor, *RegisteredDelegate);
		}		
		
		return RegisteredDelegate->DelegateHandle;
	}

	return FDelegateHandle();
}

bool UGameFrameworkComponentManager::RegisterAndCallForActorInitState(AActor* Actor, FName FeatureName, FGameplayTag RequiredState, FActorInitStateChangedBPDelegate Delegate, bool bCallImmediately /*= true*/)
{
	if (ensure(Actor && Delegate.IsBound()))
	{
		// We often register delegates before registering states
		FActorFeatureData& ActorStruct = FindOrAddActorData(Actor);

		TSharedRef<FActorFeatureRegisteredDelegate> RegisteredDelegate = MakeShared<FActorFeatureRegisteredDelegate>(MoveTemp(Delegate), FeatureName, RequiredState);
		ActorStruct.RegisteredDelegates.Add(RegisteredDelegate);

		if (bCallImmediately)
		{
			// The shared ref keeps the delegate alive in case it gets unregistered
			CallDelegateForMatchingFeatures(Actor, *RegisteredDelegate);
		}

		return true;
	}

	return false;
}

bool UGameFrameworkComponentManager::UnregisterActorInitStateDelegate(AActor* Actor, FDelegateHandle& Handle)
{
	if (Actor && Handle.IsValid())
	{
		FActorFeatureData* ActorStruct = ActorFeatureMap.Find(FObjectKey(Actor));

		if (ActorStruct)
		{
			return RemoveActorFeatureDelegateFromList(ActorStruct->RegisteredDelegates, Handle);
		}
	}

	return false;
}

bool UGameFrameworkComponentManager::UnregisterActorInitStateDelegate(AActor* Actor, FActorInitStateChangedBPDelegate DelegateToRemove)
{
	if (Actor && DelegateToRemove.IsBound())
	{
		FActorFeatureData* ActorStruct = ActorFeatureMap.Find(FObjectKey(Actor));

		if (ActorStruct)
		{
			return RemoveActorFeatureDelegateFromList(ActorStruct->RegisteredDelegates, DelegateToRemove);
		}
	}

	return false;
}

FDelegateHandle UGameFrameworkComponentManager::RegisterAndCallForClassInitState(const TSoftClassPtr<AActor>& ActorClass, FName FeatureName, FGameplayTag RequiredState, FActorInitStateChangedDelegate Delegate, bool bCallImmediately)
{
	if (ensure(!ActorClass.IsNull() && Delegate.IsBound() && !FeatureName.IsNone()))
	{
		FComponentRequestReceiverClassPath ReceiverClassPath(ActorClass);
		FActorFeatureDelegateList& RegisteredDelegates = ClassFeatureChangeDelegates.FindOrAdd(ReceiverClassPath);

		TSharedRef<FActorFeatureRegisteredDelegate> RegisteredDelegate = MakeShared<FActorFeatureRegisteredDelegate>(MoveTemp(Delegate), FeatureName, RequiredState);
		RegisteredDelegates.Add(RegisteredDelegate);
	
		if (bCallImmediately)
		{
			// A null actor class means there are no registered instances and the call is ignored
			CallDelegateForMatchingActors(ActorClass.Get(), *RegisteredDelegate);
		}

		return RegisteredDelegate->DelegateHandle;
	}

	return FDelegateHandle();
}

bool UGameFrameworkComponentManager::RegisterAndCallForClassInitState(TSoftClassPtr<AActor> ActorClass, FName FeatureName, FGameplayTag RequiredState, FActorInitStateChangedBPDelegate Delegate, bool bCallImmediately /*= true*/)
{
	if (ensure(!ActorClass.IsNull() && Delegate.IsBound() && !FeatureName.IsNone()))
	{
		FComponentRequestReceiverClassPath ReceiverClassPath(ActorClass);
		FActorFeatureDelegateList& RegisteredDelegates = ClassFeatureChangeDelegates.FindOrAdd(ReceiverClassPath);

		TSharedRef<FActorFeatureRegisteredDelegate> RegisteredDelegate = MakeShared<FActorFeatureRegisteredDelegate>(MoveTemp(Delegate), FeatureName, RequiredState);
		RegisteredDelegates.Add(RegisteredDelegate);

		if (bCallImmediately)
		{
			// A null actor class means there are no registered instances and the call is ignored
			CallDelegateForMatchingActors(ActorClass.Get(), *RegisteredDelegate);
		}

		return true;
	}

	return false;
}

bool UGameFrameworkComponentManager::UnregisterClassInitStateDelegate(const TSoftClassPtr<AActor>& ActorClass, FDelegateHandle& Handle)
{
	if (!ActorClass.IsNull() && Handle.IsValid())
	{
		FComponentRequestReceiverClassPath ReceiverClassPath(ActorClass);
		FActorFeatureDelegateList* RegisteredDelegates = ClassFeatureChangeDelegates.Find(ReceiverClassPath);

		if (RegisteredDelegates)
		{
			return RemoveActorFeatureDelegateFromList(*RegisteredDelegates, Handle);
		}
	}

	return false;
}

bool UGameFrameworkComponentManager::UnregisterClassInitStateDelegate(TSoftClassPtr<AActor> ActorClass, FActorInitStateChangedBPDelegate DelegateToRemove)
{
	if (!ActorClass.IsNull() && DelegateToRemove.IsBound())
	{
		FComponentRequestReceiverClassPath ReceiverClassPath(ActorClass);
		FActorFeatureDelegateList* RegisteredDelegates = ClassFeatureChangeDelegates.Find(ReceiverClassPath);

		if (RegisteredDelegates)
		{
			return RemoveActorFeatureDelegateFromList(*RegisteredDelegates, DelegateToRemove);
		}
	}

	return false;
}

const UGameFrameworkComponentManager::FActorFeatureState* UGameFrameworkComponentManager::FindFeatureStateStruct(const FActorFeatureData* ActorStruct, FName FeatureName, FGameplayTag RequiredState) const
{
	if (ActorStruct)
	{
		for (const FActorFeatureState& State : ActorStruct->RegisteredStates)
		{
			if (State.FeatureName == FeatureName)
			{
				if (!RequiredState.IsValid() || IsInitStateAfterOrEqual(State.CurrentState, RequiredState))
				{
					return &State;
				}
			}
		}
	}

	return nullptr;
}

void UGameFrameworkComponentManager::ProcessFeatureStateChange(AActor* Actor, const FActorFeatureState* StateChange)
{
	StateChangeQueue.Emplace(Actor, *StateChange);

	if (CurrentStateChange == INDEX_NONE)
	{
		// Start processing in order
		CurrentStateChange = 0;

		while (CurrentStateChange < StateChangeQueue.Num())
		{
			CallFeatureStateDelegates(StateChangeQueue[CurrentStateChange].Key, StateChangeQueue[CurrentStateChange].Value);
			CurrentStateChange++;
		}

		// Done processing, clear it
		StateChangeQueue.Empty();
		CurrentStateChange = INDEX_NONE;
	}
}

void UGameFrameworkComponentManager::CallFeatureStateDelegates(AActor* Actor, FActorFeatureState StateChange)
{
	FActorFeatureData* ActorStruct = ActorFeatureMap.Find(FObjectKey(Actor));
	FActorFeatureDelegateList QueuedDelegates;

	// Should only be called inside ProcessFeatureStateChange
	ensure(CurrentStateChange != INDEX_NONE);

	if (ActorStruct)
	{
		for (TSharedRef<FActorFeatureRegisteredDelegate>& DelegateRef : ActorStruct->RegisteredDelegates)
		{
			FActorFeatureRegisteredDelegate& RegisteredDelegate = *DelegateRef;
			if ((RegisteredDelegate.RequiredFeatureName.IsNone() || RegisteredDelegate.RequiredFeatureName == StateChange.FeatureName)
				&& (!RegisteredDelegate.RequiredInitState.IsValid() || IsInitStateAfterOrEqual(StateChange.CurrentState, RegisteredDelegate.RequiredInitState)))
			{
				// Queue delegates now in case the registered list changes during execution
				// If new delegates are registered, they are handled at registration time if bCallImmediately is used
				QueuedDelegates.Add(DelegateRef);
			}
		}

		UClass* ClassToCheck = Actor->GetClass();
		while (ClassToCheck)
		{
			// Now check the general class delegates
			FComponentRequestReceiverClassPath ReceiverClassPath(ClassToCheck);
			FActorFeatureDelegateList* FoundDelegates = ClassFeatureChangeDelegates.Find(ReceiverClassPath);

			if (FoundDelegates)
			{
				for (TSharedRef<FActorFeatureRegisteredDelegate>& DelegateRef : *FoundDelegates)
				{
					FActorFeatureRegisteredDelegate& RegisteredDelegate = *DelegateRef;
					if ((RegisteredDelegate.RequiredFeatureName.IsNone() || RegisteredDelegate.RequiredFeatureName == StateChange.FeatureName)
						&& (!RegisteredDelegate.RequiredInitState.IsValid() || IsInitStateAfterOrEqual(StateChange.CurrentState, RegisteredDelegate.RequiredInitState)))
					{
						QueuedDelegates.Add(DelegateRef);
					}

				}
			}

			ClassToCheck = ClassToCheck->GetSuperClass();
		}
	}

	// Now execute the queued delegates, if they are removed Execute will skip them
	for (TSharedRef<FActorFeatureRegisteredDelegate>& QueuedDelegate : QueuedDelegates)
	{
		QueuedDelegate->Execute(Actor, StateChange.FeatureName, StateChange.Implementer.Get(), StateChange.CurrentState);
	}
}

void UGameFrameworkComponentManager::CallDelegateForMatchingFeatures(AActor* Actor, FActorFeatureRegisteredDelegate& RegisteredDelegate)
{
	FActorFeatureData* ActorStruct = ActorFeatureMap.Find(FObjectKey(Actor));

	if (ActorStruct == nullptr)
	{
		return;
	}

	// If feature is specified, just call the one
	if (!RegisteredDelegate.RequiredFeatureName.IsNone())
	{
		const FActorFeatureState* FoundStruct = FindFeatureStateStruct(ActorStruct, RegisteredDelegate.RequiredFeatureName, RegisteredDelegate.RequiredInitState);

		if (FoundStruct)
		{
			RegisteredDelegate.Execute(Actor, FoundStruct->FeatureName, FoundStruct->Implementer.Get(), FoundStruct->CurrentState);
		}

		return;
	}
	
	// If feature is not specified, iterate and run on all valid ones
	TArray<FActorFeatureState> QueuedStates;
	for (FActorFeatureState& FeatureState : ActorStruct->RegisteredStates)
	{
		if (!RegisteredDelegate.RequiredInitState.IsValid() || IsInitStateAfterOrEqual(FeatureState.CurrentState, RegisteredDelegate.RequiredInitState))
		{
			// Make a copy in case the state memory is invalidated
			QueuedStates.Add(FeatureState);
		}
	}

	for (FActorFeatureState& FeatureState : QueuedStates)
	{
		RegisteredDelegate.Execute(Actor, FeatureState.FeatureName, FeatureState.Implementer.Get(), FeatureState.CurrentState);
	}
}

void UGameFrameworkComponentManager::CallDelegateForMatchingActors(UClass* ActorClass, FActorFeatureRegisteredDelegate& RegisteredDelegate)
{
	TArray<AActor*> MatchingActors;

	if (ActorClass == nullptr)
	{
		return;
	}

	for (TPair<FObjectKey, FActorFeatureData>& Pair : ActorFeatureMap)
	{
		UClass* CheckActorClass = Pair.Value.ActorClass.Get();

		if (CheckActorClass && CheckActorClass->IsChildOf(ActorClass))
		{
			AActor* FoundActor = Cast<AActor>(Pair.Key.ResolveObjectPtr());
			if (FoundActor)
			{
				MatchingActors.Add(FoundActor);
			}
		}
	}

	// Iterate actor list before calling any delegates as delegates could change it
	for (AActor* Actor : MatchingActors)
	{
		CallDelegateForMatchingFeatures(Actor, RegisteredDelegate);
	}
}

UGameFrameworkComponentManager::FActorFeatureData& UGameFrameworkComponentManager::FindOrAddActorData(AActor* Actor)
{
	check(Actor);

	FActorFeatureData& ActorStruct = ActorFeatureMap.FindOrAdd(FObjectKey(Actor));
	if (!ActorStruct.ActorClass.IsValid())
	{
		ActorStruct.ActorClass = Actor->GetClass();
	}
	return ActorStruct;
}

bool UGameFrameworkComponentManager::RemoveActorFeatureDelegateFromList(FActorFeatureDelegateList& DelegateList, FDelegateHandle& SearchHandle) const
{
	for (int32 i = DelegateList.Num() - 1; i >= 0; i--)
	{
		if (DelegateList[i]->DelegateHandle == SearchHandle)
		{
			DelegateList[i]->bRemoved = true;
			DelegateList.RemoveAt(i);
			SearchHandle.Reset();
			return true;
		}
	}

	return false;
}

bool UGameFrameworkComponentManager::RemoveActorFeatureDelegateFromList(FActorFeatureDelegateList& DelegateList, FActorInitStateChangedBPDelegate SearchDelegate) const
{
	for (int32 i = DelegateList.Num() - 1; i >= 0; i--)
	{
		if (DelegateList[i]->BPDelegate == SearchDelegate)
		{
			DelegateList[i]->bRemoved = true;
			DelegateList.RemoveAt(i);
			return true;
		}
	}

	return false;
}

void UGameFrameworkComponentManager::PruneUnboundDelegates(TConstArrayView<UPackage*> Packages)
{
	// With merged modules, dynamic libraries can unmount with the code backing delegate instances, including validity checks.
	// As a result, before a library unmount, we have to prune delegates registered to destroyed objects, which we detect by checking if they're still bound.
	auto PruneDelegateList = [](FActorFeatureDelegateList& DelegateList)
	{
		for (auto DelegateIter = DelegateList.CreateIterator(); DelegateIter; ++DelegateIter)
		{
			if (!(*DelegateIter)->IsBound())
			{
				UE_LOGF(LogModularGameplay, Verbose, "PruneUnboundDelegates: removing unbound delegate");
				(*DelegateIter)->bRemoved = true;
				DelegateIter.RemoveCurrent();
			}
		}
	};

	for (auto ClassFeatureChangeIter = ClassFeatureChangeDelegates.CreateIterator(); ClassFeatureChangeIter; ++ClassFeatureChangeIter)
	{
		PruneDelegateList(ClassFeatureChangeIter->Value);
	}

	for (auto ActorFeatureIter = ActorFeatureMap.CreateIterator(); ActorFeatureIter; ++ActorFeatureIter)
	{
		PruneDelegateList(ActorFeatureIter->Value.RegisteredDelegates);
	}

	for (auto ReceiverClassToEventIter = ReceiverClassToEventMap.CreateIterator(); ReceiverClassToEventIter; ++ReceiverClassToEventIter)
	{
		for (auto ExtensionHandlerEventIter = ReceiverClassToEventIter->Value.CreateIterator(); ExtensionHandlerEventIter; ++ExtensionHandlerEventIter)
		{
			if (!ExtensionHandlerEventIter->Value->Delegate.IsBound())
			{
				UE_LOGF(LogModularGameplay, Verbose, "PruneUnboundDelegates: removing unbound delegate");
				ExtensionHandlerEventIter->Value->bRemoved = true;
				ExtensionHandlerEventIter.RemoveCurrent();
			}
		}
	}
}
