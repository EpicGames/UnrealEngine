// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/GameFrameworkInitStateInterface.h"
#include "Components/GameFrameworkComponentDelegates.h"
#include "Components/GameFrameworkComponentManager.h"
#include "GameFramework/Actor.h"
#include "ModularGameplayLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFrameworkInitStateInterface)


AActor* IGameFrameworkInitStateInterface::GetOwningActor() const
{
	// Removing const because most AActor functions can't handle const
	AActor* FoundActor = const_cast<AActor*>(Cast<AActor>(this));

	if (!FoundActor)
	{
		const UActorComponent* FoundComponent = Cast<UActorComponent>(this);
		if (FoundComponent)
		{
			FoundActor = FoundComponent->GetOwner();
		}
	}

	if (ensure(FoundActor))
	{
		// Subclasses must implement this if they are not actors or components
		return FoundActor;
	}

	return nullptr;
}

UGameFrameworkComponentManager* IGameFrameworkInitStateInterface::GetComponentManager() const
{
	return UGameFrameworkComponentManager::GetForActor(GetOwningActor());
}

FGameplayTag IGameFrameworkInitStateInterface::GetInitState() const
{
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);
	if (Manager)
	{
		return Manager->GetInitStateForFeature(MyActor, GetFeatureName());
	}

	return FGameplayTag();
}

bool IGameFrameworkInitStateInterface::HasReachedInitState(FGameplayTag DesiredState) const
{
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);
	if (Manager)
	{
		return Manager->HasFeatureReachedInitState(MyActor, GetFeatureName(), DesiredState);
	}

	return false;
}

bool IGameFrameworkInitStateInterface::TryToChangeInitState(FGameplayTag DesiredState)
{
	UObject* ThisObject = Cast<UObject>(this);
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);
	const FName MyFeatureName = GetFeatureName();

	if (!Manager || !ThisObject || !MyActor)
	{
		return false;	
	}

	FGameplayTag CurrentState = Manager->GetInitStateForFeature(MyActor, MyFeatureName);
	
	// If we're already in that state, just return
	if (CurrentState == DesiredState)
	{
		return false;
	}

	if (!CanChangeInitState(Manager, CurrentState, DesiredState))
	{
		UE_LOGF(LogModularGameplay, Verbose, "TryToChangeInitState: Cannot transition %ls:%ls (role %d) from %ls to %ls",
			*MyActor->GetName(), *MyFeatureName.ToString(), MyActor->GetLocalRole(), *CurrentState.ToString(), *DesiredState.ToString());
		return false;
	}

	UE_LOGF(LogModularGameplay, Verbose, "TryToChangeInitState: Transitioning %ls:%ls (role %d) from %ls to %ls",
		*MyActor->GetName(), *MyFeatureName.ToString(), MyActor->GetLocalRole(), *CurrentState.ToString(), *DesiredState.ToString());

	// Perform the local change
	HandleChangeInitState(Manager, CurrentState, DesiredState);

	// The local change has completed, notify the system to register change and execute callbacks
	return ensure(Manager->ChangeFeatureInitState(MyActor, MyFeatureName, ThisObject, DesiredState));
}

FGameplayTag IGameFrameworkInitStateInterface::ContinueInitStateChain(const TArray<FGameplayTag>& InitStateChain)
{
	UObject* ThisObject = Cast<UObject>(this);
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);
	const FName MyFeatureName = GetFeatureName();

	if (!Manager || !ThisObject || !MyActor)
	{
		return FGameplayTag();
	}

	int32 ChainIndex = 0;
	FGameplayTag CurrentState = Manager->GetInitStateForFeature(MyActor, MyFeatureName);

	// For each state in chain before the last, see if we can transition to the next state
	while (ChainIndex < InitStateChain.Num() - 1)
	{
		if (CurrentState == InitStateChain[ChainIndex])
		{
			FGameplayTag DesiredState = InitStateChain[ChainIndex + 1];
			if (CanChangeInitState(Manager, CurrentState, DesiredState))
			{
				UE_LOGF(LogModularGameplay, Verbose, "ContinueInitStateChain: Transitioning %ls:%ls (role %d) from %ls to %ls",
					*MyActor->GetName(), *MyFeatureName.ToString(), MyActor->GetLocalRole(), *CurrentState.ToString(), *DesiredState.ToString());

				// Perform the local change
				HandleChangeInitState(Manager, CurrentState, DesiredState);

				// The local change has completed, notify the system to register change and execute callbacks
				ensure(Manager->ChangeFeatureInitState(MyActor, MyFeatureName, ThisObject, DesiredState));

				// Update state and check again
				CurrentState = Manager->GetInitStateForFeature(MyActor, MyFeatureName);
			}
			else
			{
				UE_LOGF(LogModularGameplay, Verbose, "ContinueInitStateChain: Cannot transition %ls:%ls (role %d) from %ls to %ls",
					*MyActor->GetName(), *MyFeatureName.ToString(), MyActor->GetLocalRole(), *CurrentState.ToString(), *DesiredState.ToString());
			}
		}

		ChainIndex++;
	}

	return CurrentState;
}

void IGameFrameworkInitStateInterface::CheckDefaultInitializationForImplementers()
{
	UObject* ThisObject = Cast<UObject>(this);
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);
	const FName MyFeatureName = GetFeatureName();

	if (Manager)
	{
		TArray<UObject*> Implementers;
		Manager->GetAllFeatureImplementers(Implementers, MyActor, FGameplayTag(), MyFeatureName);

		for (UObject* Implementer : Implementers)
		{
			if (IGameFrameworkInitStateInterface* ImplementerInterface = Cast<IGameFrameworkInitStateInterface>(Implementer))
			{
				ImplementerInterface->CheckDefaultInitialization();
			}
		}
	}
}


void IGameFrameworkInitStateInterface::RegisterInitStateFeature()
{
	UObject* ThisObject = Cast<UObject>(this);
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);
	const FName MyFeatureName = GetFeatureName();

	if (MyActor && Manager)
	{
		// Manager will be null if this isn't in a game world
		Manager->RegisterFeatureImplementer(MyActor, MyFeatureName, ThisObject);
	}
}

void IGameFrameworkInitStateInterface::UnregisterInitStateFeature()
{
	UObject* ThisObject = Cast<UObject>(this);
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);

	if (Manager)
	{
		if (ThisObject == MyActor)
		{
			// This will clear all the feature states and delegates
			Manager->RemoveActorFeatureData(MyActor);
		}
		else
		{
			Manager->RemoveFeatureImplementer(MyActor, ThisObject);
		}

		if (ActorInitStateChangedHandle.IsValid())
		{
			Manager->UnregisterActorInitStateDelegate(MyActor, ActorInitStateChangedHandle);
		}
	}
}

void IGameFrameworkInitStateInterface::BindOnActorInitStateChanged(FName FeatureName, FGameplayTag RequiredState, bool bCallIfReached)
{
	UObject* ThisObject = Cast<UObject>(this);
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);

	if (ensure(MyActor && Manager))
	{
		// Bind as a weak lambda because this is not a UObject but is guaranteed to be valid as long as ThisObject is
		FActorInitStateChangedDelegate Delegate = FActorInitStateChangedDelegate::CreateWeakLambda(ThisObject,
			[this](const FActorInitStateChangedParams& Params)
			{
				this->OnActorInitStateChanged(Params);
			});

		ActorInitStateChangedHandle = Manager->RegisterAndCallForActorInitState(MyActor, FeatureName, RequiredState, MoveTemp(Delegate), bCallIfReached);
	}
}

bool IGameFrameworkInitStateInterface::RegisterAndCallForInitStateChange(FGameplayTag RequiredState, FActorInitStateChangedBPDelegate Delegate, bool bCallImmediately)
{
	UObject* ThisObject = Cast<UObject>(this);
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);
	const FName MyFeatureName = GetFeatureName();

	if (ensure(MyActor && Manager))
	{
		return Manager->RegisterAndCallForActorInitState(MyActor, MyFeatureName, RequiredState, Delegate, bCallImmediately);
	}

	return false;
}

bool IGameFrameworkInitStateInterface::UnregisterInitStateDelegate(FActorInitStateChangedBPDelegate Delegate)
{
	UObject* ThisObject = Cast<UObject>(this);
	AActor* MyActor = GetOwningActor();
	UGameFrameworkComponentManager* Manager = UGameFrameworkComponentManager::GetForActor(MyActor);

	if (ensure(MyActor && Manager))
	{
		return Manager->UnregisterActorInitStateDelegate(MyActor, Delegate);
	}

	return false;
}

FString IGameFrameworkInitStateInterface::GetDebugState() const
{
	return FString::Printf(TEXT("CurrentState: %s"), *GetInitState().ToString());
}
