// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/GameStateComponent.h"
#include "Components/GameFrameworkComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameStateComponent)

UGameStateComponent::UGameStateComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
