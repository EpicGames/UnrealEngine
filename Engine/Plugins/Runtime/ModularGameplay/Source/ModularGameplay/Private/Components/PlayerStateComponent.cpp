// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PlayerStateComponent.h"
#include "Components/GameFrameworkComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerStateComponent)

UPlayerStateComponent::UPlayerStateComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
