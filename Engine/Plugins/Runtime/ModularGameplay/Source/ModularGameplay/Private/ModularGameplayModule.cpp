// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularGameplayModule.h"
#include "ModularGameplayLogs.h"

class FModularGameplayModule : public IModularGameplayModule
{
};

IMPLEMENT_MODULE(FModularGameplayModule, ModularGameplay)

DEFINE_LOG_CATEGORY(LogModularGameplay);