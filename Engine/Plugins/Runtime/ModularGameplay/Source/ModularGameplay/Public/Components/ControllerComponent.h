// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/PlayerController.h"
#include "GameFrameworkComponent.h"
#include "ControllerComponent.generated.h"

#define UE_API MODULARGAMEPLAY_API

/**
 * ControllerComponent is an actor component made for AController and receives controller events.
 */
UCLASS(MinimalAPI)
class UControllerComponent : public UGameFrameworkComponent
{
	GENERATED_BODY()

public:
	
	UE_API UControllerComponent(const FObjectInitializer& ObjectInitializer);

	/** Gets the controller that owns the component, this will always be valid during gameplay but can return null in the editor */
	template <class T>
	T* GetController() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, AController>::Value, "'T' template parameter to GetController must be derived from AController");
		return Cast<T>(GetOwner());
	}

	template <class T>
	T* GetControllerChecked() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, AController>::Value, "'T' template parameter to GetControllerChecked must be derived from AController");
		return CastChecked<T>(GetOwner());
	}

	//////////////////////////////////////////////////////////////////////////////
	// Controller accessors
	// Usable for any type of AController owner, only valid if called during gameplay
	//////////////////////////////////////////////////////////////////////////////

	/** Returns the pawn that is currently possessed by the owning controller, will often return null */
	template <class T>
	T* GetPawn() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APawn>::Value, "'T' template parameter to GetPawn must be derived from APawn");
		return Cast<T>(GetControllerChecked<AController>()->GetPawn());
	}

	/** Returns the actor that is serving as the current view target for the owning controller */
	template <class T>
	T* GetViewTarget() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, AActor>::Value, "'T' template parameter to GetViewTarget must be derived from APawn");
		return Cast<T>(GetControllerChecked<AController>()->GetViewTarget());
	}

	/** If this controller is possessing a pawn return the pawn, if not return the view target */
	template <class T>
	T* GetPawnOrViewTarget() const
	{
		if (T* Pawn = GetPawn<T>())
		{
			return Pawn;
		}
		else
		{
			return GetViewTarget<T>();
		}
	}

	/** Returns the player state attached to this controller if there is one */
	template <class T>
	T* GetPlayerState() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APlayerState>::Value, "'T' template parameter to GetPlayerState must be derived from APlayerState");
		return GetControllerChecked<AController>()->GetPlayerState<T>();
	}

	/** Returns true if the owning controller is considered to be local */
	UE_API bool IsLocalController() const;

	/** Returns the point of view for either a player or controlled pawn */
	UE_API void GetPlayerViewPoint(FVector& Location, FRotator& Rotation) const;

	//////////////////////////////////////////////////////////////////////////////
	// PlayerController accessors
	// Only returns correct values for APlayerController owners
	//////////////////////////////////////////////////////////////////////////////

	template <class T>
	T* GetPlayer() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, UPlayer>::Value, "'T' template parameter to GetPlayer must be derived from UPlayer");
		APlayerController* PC = Cast<APlayerController>(GetOwner());
		return PC ? Cast<T>(PC->Player) : nullptr;
	}

public:

	//////////////////////////////////////////////////////////////////////////////
	// PlayerController events
	// These only happen if the controller is a PlayerController
	//////////////////////////////////////////////////////////////////////////////

	/** Called after the PlayerController's viewport/net connection is associated with this player controller. */
	virtual void ReceivedPlayer() {}

	/** PlayerTick is only called if the PlayerController has a PlayerInput object. Therefore, it will only be called for locally controlled PlayerControllers. */
	virtual void PlayerTick(float DeltaTime) {}
};

#undef UE_API
