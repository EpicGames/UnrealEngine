// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "GameFrameworkComponent.generated.h"

#define UE_API MODULARGAMEPLAY_API

/**
 * GameFrameworkComponent is a base class for actor components made for the basic game framework classes.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, HideCategories=(Trigger, PhysicsVolume))
class UGameFrameworkComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	
	UE_API UGameFrameworkComponent(const FObjectInitializer& ObjectInitializer);

	/** Gets the game instance this component is a part of, this will return null if not called during normal gameplay */
	template <class T>
	T* GetGameInstance() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, UGameInstance>::Value, "'T' template parameter to GetGameInstance must be derived from UGameInstance");
		AActor* Owner = GetOwner();
		return Owner ? Owner->GetGameInstance<T>() : nullptr;
	}

	template <class T>
	T* GetGameInstanceChecked() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, UGameInstance>::Value, "'T' template parameter to GetGameInstance must be derived from UGameInstance");
		AActor* Owner = GetOwner();
		check(Owner);
		T* GameInstance = Owner->GetGameInstance<T>();
		check(GameInstance);
		return GameInstance;
	}

	/** Returns true if the owner's role is ROLE_Authority */
	UE_API bool HasAuthority() const;

	/** Returns the world's timer manager */
	UE_API class FTimerManager& GetWorldTimerManager() const;
};

/**
 * Iterator for registered components on an actor
 */
template <typename T>
class TComponentIterator
{
public:
	explicit TComponentIterator(AActor* OwnerActor)
		: CompIndex(-1)
	{
		if (IsValid(OwnerActor))
		{
			OwnerActor->GetComponents(AllComponents);
		}

		Advance();
	}

	inline void operator++()
	{
		Advance();
	}

	inline explicit operator bool() const
	{
		return AllComponents.IsValidIndex(CompIndex);
	}

	inline bool operator!() const
	{
		return !(bool)*this;
	}

	inline T* operator*() const
	{
		return GetComponent();
	}

	inline T* operator->() const
	{
		return GetComponent();
	}

protected:
	/** Gets the current component */
	inline T* GetComponent() const
	{
		return AllComponents[CompIndex];
	}

	/** Moves the iterator to the next valid component */
	inline bool Advance()
	{
		while (++CompIndex < AllComponents.Num())
		{
			T* Comp = GetComponent();
			check(Comp);
			if (Comp->IsRegistered())
			{
				checkf(IsValid(Comp), TEXT("Registered game framework component was pending kill! Comp: %s"), *GetPathNameSafe(Comp));
				return true;
			}
		}

		return false;
	}

private:
	/** Results from GetComponents */
	TInlineComponentArray<T*> AllComponents;

	/** Index of the current element in the componnet array */
	int32 CompIndex;

	inline bool operator==(const TComponentIterator& Other) const { return CompIndex == Other.CompIndex; }
	inline bool operator!=(const TComponentIterator& Other) const { return CompIndex != Other.CompIndex; }
};

/**
 * Const iterator for registered components on an actor
 */
template <typename T>
class TConstComponentIterator
{
public:
	explicit TConstComponentIterator(const AActor* OwnerActor)
		: CompIndex(-1)
	{
		if (IsValid(OwnerActor))
		{
			OwnerActor->GetComponents(AllComponents);
		}

		Advance();
	}

	inline void operator++()
	{
		Advance();
	}

	inline explicit operator bool() const
	{
		return AllComponents.IsValidIndex(CompIndex);
	}

	inline bool operator!() const
	{
		return !(bool)*this;
	}

	inline const T* operator*() const
	{
		return GetComponent();
	}

	inline const T* operator->() const
	{
		return GetComponent();
	}

protected:
	/** Gets the current component */
	inline const T* GetComponent() const
	{
		return AllComponents[CompIndex];
	}

	/** Moves the iterator to the next valid component */
	inline bool Advance()
	{
		while (++CompIndex < AllComponents.Num())
		{
			const T* Comp = GetComponent();
			check(Comp);
			if (Comp->IsRegistered())
			{
				checkf(IsValidChecked(Comp), TEXT("Registered game framework component was invalid! Comp: %s"), *GetPathNameSafe(Comp));
				return true;
			}
		}

		return false;
	}

private:
	/** Results from GetComponents */
	TInlineComponentArray<const T*> AllComponents;

	/** Index of the current element in the componnet array */
	int32 CompIndex;

	inline bool operator==(const TConstComponentIterator& Other) const { return CompIndex == Other.CompIndex; }
	inline bool operator!=(const TConstComponentIterator& Other) const { return CompIndex != Other.CompIndex; }
};

#undef UE_API
