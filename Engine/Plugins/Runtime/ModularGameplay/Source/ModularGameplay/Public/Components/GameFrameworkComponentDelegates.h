// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "GameFrameworkComponentDelegates.generated.h"

/** Parameters struct for Init State change functions */
USTRUCT(BlueprintType)
struct FActorInitStateChangedParams
{
	GENERATED_BODY()

	FActorInitStateChangedParams()
		: OwningActor(nullptr)
		, Implementer(nullptr)
	{}

	FActorInitStateChangedParams(AActor* InOwningActor, FName InFeatureName, UObject* InImplementer, FGameplayTag InFeatureState)
		: OwningActor(InOwningActor)
		, FeatureName(InFeatureName)
		, Implementer(InImplementer)
		, FeatureState(InFeatureState)
	{}

	/** The actor owning the feature that changed */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	TObjectPtr<AActor> OwningActor;

	/** Name of the feature that changed */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	FName FeatureName;

	/** The object (often a component) that implements the feature */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	TObjectPtr<UObject> Implementer;

	/** The new state of the feature */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	FGameplayTag FeatureState;
};

/** Blueprint delegate called when an actor feature changes init state */
DECLARE_DYNAMIC_DELEGATE_OneParam(FActorInitStateChangedBPDelegate, const FActorInitStateChangedParams&, Params);
