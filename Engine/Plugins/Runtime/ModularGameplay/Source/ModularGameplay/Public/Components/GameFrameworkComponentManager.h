// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"
#include "Templates/SubclassOf.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "GameFrameworkComponentDelegates.h"

#include "GameFrameworkComponentManager.generated.h"

#define UE_API MODULARGAMEPLAY_API

class UGameFrameworkComponentManager;

class AActor;
class UActorComponent;
class UGameFrameworkComponent;


// Bitflag enum to control how this AddComponents action should actually run
UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EGameFrameworkAddComponentFlags : uint8
{
	// If 0, no specific rule is applied and the component will be always created on the receiver"),
	None =						0			UMETA(Hidden),
	AddUnique =					0b00000001	UMETA(DisplayName = "Add Unique", ToolTip="If AddUnique is set, it will be added only if there is no existing component on ActorInstance of ComponentClass."),
	AddIfNotChild =				0b00000010	UMETA(DisplayName = "Add Unique If Not Child", ToolTip = "If AddIfNotChild is set, it will be added only if ComponentClass is not a child of an existing component on ActorInstance."),
	UseAutoGeneratedName =		0b00000100	UMETA(DisplayName = "Use Auto Generated Name", ToolTip = "If UseAutoGeneratedName is set, it will generate a new name and not re-use the class name directly (which can lead to component recycling)."),
};
ENUM_CLASS_FLAGS(EGameFrameworkAddComponentFlags);

enum class EGameFrameworkAddComponentResult : uint8
{
	Success,
	Failed
};
/** 
 * A handle for a request to put components or call a delegate for an extensible actor class.
 * When this handle is destroyed, it will remove the associated request from the system.
 */
struct FComponentRequestHandle
{
	FComponentRequestHandle(const TWeakObjectPtr<UGameFrameworkComponentManager>& InOwningManager, const TSoftClassPtr<AActor>& InReceiverClass, const TSubclassOf<UActorComponent>& InComponentClass)
		: OwningManager(InOwningManager)
		, ReceiverClass(InReceiverClass)
		, ComponentClass(InComponentClass)
	{}

	FComponentRequestHandle(const TWeakObjectPtr<UGameFrameworkComponentManager>& InOwningManager, const TSoftClassPtr<AActor>& InReceiverClass, FDelegateHandle InExtensionHandle)
		: OwningManager(InOwningManager)
		, ReceiverClass(InReceiverClass)
		, ExtensionHandle(InExtensionHandle)
	{}

	MODULARGAMEPLAY_API ~FComponentRequestHandle();

	/** Returns true if the manager that this request is for still exists */
	MODULARGAMEPLAY_API bool IsValid() const;

private:
	/** The manager that this request was for */
	TWeakObjectPtr<UGameFrameworkComponentManager> OwningManager;

	/** The class of actor to put components */
	TSoftClassPtr<AActor> ReceiverClass;

	/** The class of component to put on actors */
	TSubclassOf<UActorComponent> ComponentClass;

	/** A handle to an extension delegate to run */
	FDelegateHandle ExtensionHandle;
};


/** Native delegate called when an actor feature changes init state */
DECLARE_DELEGATE_OneParam(FActorInitStateChangedDelegate, const FActorInitStateChangedParams&);

/** 
 * GameFrameworkComponentManager
 *
 * A manager to handle putting components on actors as they come and go.
 * Put in a request to instantiate components of a given class on actors of a given class and they will automatically be made for them as the actors are spawned.
 * Submit delegate handlers to listen for actors of a given class. Those handlers will automatically run when actors of a given class or registered as receivers or game events are sent.
 * Actors must opt-in to this behavior by calling AddReceiver/RemoveReceiver for themselves when they are ready to receive the components and when they want to remove them.
 * Any actors that are in memory when a request is made will automatically get the components, and any in memory when a request is removed will lose the components immediately.
 * Requests are reference counted, so if multiple requests are made for the same actor class and component class, only one component will be added and that component wont be removed until all requests are removed.
 */
UCLASS(MinimalAPI)
class UGameFrameworkComponentManager : public UGameInstanceSubsystem
{
	GENERATED_BODY()

public:
	/** Delegate types for extension handlers */
	DECLARE_DELEGATE_TwoParams(FExtensionHandlerDelegate, AActor*, FName);

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	/** Utility to get this manager from an actor, will return null if actor is null or not in a world */
	static UE_API UGameFrameworkComponentManager* GetForActor(const AActor* Actor, bool bOnlyGameWorlds = true);

	/** Adds an actor as a receiver for components. If it passes the actorclass filter on requests it will get the components. */
	UFUNCTION(BlueprintCallable, Category="Gameplay", meta=(DefaultToSelf="Receiver", AdvancedDisplay=1))
	UE_API void AddReceiver(AActor* Receiver, bool bAddOnlyInGameWorlds = true);

	/** Removes an actor as a receiver for components. */
	UFUNCTION(BlueprintCallable, Category="Gameplay", meta=(DefaultToSelf="Receiver"))
	UE_API void RemoveReceiver(AActor* Receiver);

	/** Adds an actor as a receiver for components (automatically finding the manager for the actor's  game instance). If it passes the actorclass filter on requests it will get the components. */
	static UE_API void AddGameFrameworkComponentReceiver(AActor* Receiver, bool bAddOnlyInGameWorlds = true);

	/** Removes an actor as a receiver for components (automatically finding the manager for the actor's game instance). */
	static UE_API void RemoveGameFrameworkComponentReceiver(AActor* Receiver);

	/** 
	* Adds a request to instantiate components on actors of the given classes. 
	* 
	* @param ReceiverClass		The class on which we should add the component onto
	* @param ComponentClass		The component class to add on the new instance
	* @param AdditionFlags (optional)	Set of flags to considered at component creation
	* 
	* @return Returns a handle that will keep the request "alive" until all handles for the request are destructed, at which point the request is removed.
	*/
	UE_API TSharedPtr<FComponentRequestHandle> AddComponentRequest(const TSoftClassPtr<AActor>& ReceiverClass, TSubclassOf<UActorComponent> ComponentClass, const EGameFrameworkAddComponentFlags AdditionFlags = EGameFrameworkAddComponentFlags::None);


	//////////////////////////////////////////////////////////////////////////////////////////////
	// The extension system allows registering for arbitrary event callbacks on receiver actors.
	// These are the default events but games can define, send, and listen for their own.

	/** AddReceiver was called for a registered class and components were added, called early in initialization */
	static UE_API FName NAME_ReceiverAdded;

	/** RemoveReceiver was called for a registered class and components were removed, normally called from EndPlay */
	static UE_API FName NAME_ReceiverRemoved;

	/** A new extension handler was added */
	static UE_API FName NAME_ExtensionAdded;

	/** An extension handler was removed by a freed request handle */
	static UE_API FName NAME_ExtensionRemoved;

	/** 
	 * Game-specific event indicating an actor is mostly initialized and ready for extension. 
	 * All extensible games are expected to send this event at the appropriate actor-specific point, as plugins may be listening for it.
	 */
	static UE_API FName NAME_GameActorReady;


	/** Adds an extension handler to run on actors of the given class. Returns a handle that will keep the handler "alive" until it is destructed, at which point the delegate is removed */
	UE_API TSharedPtr<FComponentRequestHandle> AddExtensionHandler(const TSoftClassPtr<AActor>& ReceiverClass, FExtensionHandlerDelegate ExtensionHandler);

	/** Sends an arbitrary extension event that can be listened for by other systems */
	UFUNCTION(BlueprintCallable, Category = "Gameplay", meta = (DefaultToSelf = "Receiver", AdvancedDisplay = 1))
	UE_API void SendExtensionEvent(AActor* Receiver, FName EventName, bool bOnlyInGameWorlds = true);

	/** Sends an arbitrary extension event that can be listened for by other systems */
	static UE_API void SendGameFrameworkComponentExtensionEvent(AActor* Receiver, const FName& EventName, bool bOnlyInGameWorlds = true);

	static UE_API void AddReferencedObjects(UObject* InThis, class FReferenceCollector& Collector);

#if !UE_BUILD_SHIPPING
	static UE_API void DumpGameFrameworkComponentManagers();
#endif // !UE_BUILD_SHIPPING

private:
#if WITH_EDITORONLY_DATA
	UE_API void PostGC();
#endif

	UE_API void AddReceiverInternal(AActor* Receiver);
	UE_API void RemoveReceiverInternal(AActor* Receiver);
	UE_API void SendExtensionEventInternal(AActor* Receiver, const FName& EventName);

	/** Called by FComponentRequestHandle's destructor to remove a request for components to be created. */
	UE_API void RemoveComponentRequest(const TSoftClassPtr<AActor>& ReceiverClass, TSubclassOf<UActorComponent> ComponentClass);

	/** Called by FComponentRequestHandle's destructor to remove a handler from the system. */
	UE_API void RemoveExtensionHandler(const TSoftClassPtr<AActor>& ReceiverClass, FDelegateHandle DelegateHandle);

	/** Creates an instance of a component on an actor */
	UE_API EGameFrameworkAddComponentResult CreateComponentOnInstance(AActor* ActorInstance, TSubclassOf<UActorComponent> ComponentClass, const EGameFrameworkAddComponentFlags AdditionFlags = EGameFrameworkAddComponentFlags::None);

	/** Removes an instance of a component on an actor */
	UE_API void DestroyInstancedComponent(UActorComponent* Component);

	/** A pair holding a receiver class and AdditionFlags */
	struct FComponentRequestInfo
	{
		FComponentRequestInfo(UClass* InClass, const EGameFrameworkAddComponentFlags InAdditionFlags = EGameFrameworkAddComponentFlags::None)
			: Class(InClass)
			, AdditionFlags(InAdditionFlags)
		{}

		bool operator==(const FComponentRequestInfo& Other) const
		{
			return Class == Other.Class;
		}

		friend inline uint32 GetTypeHash(const FComponentRequestInfo& Request)
		{
			return GetTypeHash(Request.Class);
		}

		// The receiver class
		TObjectPtr<UClass> Class;

		// Observe these rules when adding the component, if any
		EGameFrameworkAddComponentFlags AdditionFlags = EGameFrameworkAddComponentFlags::None;
	};

	/** A list of FNames to represent an object path. Used for fast hashing and comparison of paths */
	struct FComponentRequestReceiverClassPath
	{
		// Usually only 2 elements long. Ex [UPackage FName, UClass FName]
		TArray<FName, TInlineAllocator<2>> Path;

		FComponentRequestReceiverClassPath() {}

		FComponentRequestReceiverClassPath(UClass* InClass)
		{
			check(InClass);
			for (UObject* Obj = InClass; Obj; Obj = Obj->GetOuter())
			{
				Path.Insert(Obj->GetFName(), 0);
			}
		}

		FComponentRequestReceiverClassPath(const TSoftClassPtr<AActor>& InSoftClassPtr)
		{
			TArray<FString> StringPath;
			InSoftClassPtr.ToString().ParseIntoArray(StringPath, TEXT("."));
			Path.Reserve(StringPath.Num());
			for (const FString& StringPathElement : StringPath)
			{
				Path.Add(FName(*StringPathElement));
			}
		}

#if !UE_BUILD_SHIPPING
		FString ToDebugString() const
		{
			FString ReturnString;
			if (Path.Num() > 0)
			{
				ReturnString = Path[0].ToString();
				for (int32 PathIdx = 1; PathIdx < Path.Num(); ++PathIdx)
				{
					ReturnString += TEXT(".") + Path[PathIdx].ToString();
				}
			}

			return ReturnString;
		}
#endif // !UE_BUILD_SHIPPING

		bool operator==(const FComponentRequestReceiverClassPath& Other) const
		{
			return Path == Other.Path;
		}

		friend inline uint32 GetTypeHash(const FComponentRequestReceiverClassPath& Request)
		{
			uint32 ReturnHash = 0;
			for (const FName& PathElement : Request.Path)
			{
				ReturnHash ^= GetTypeHash(PathElement);
			}
			return ReturnHash;
		}
	};

	/** A pair of classes that describe a request. Together these form a key that is used to batch together requests that are identical and reference count them */
	struct FComponentRequest
	{
		FComponentRequestReceiverClassPath ReceiverClassPath;
		UClass* ComponentClass;

		bool operator==(const FComponentRequest& Other) const
		{
			return ReceiverClassPath == Other.ReceiverClassPath && ComponentClass == Other.ComponentClass;
		}

		friend inline uint32 GetTypeHash(const FComponentRequest& Request)
		{
			return GetTypeHash(Request.ReceiverClassPath) ^ GetTypeHash(Request.ComponentClass);
		}
	};

	/** Stores a delegate as shared ref to avoid deallocation during execution */
	struct FExtensionHandlerRegisteredDelegate
	{
		/** Delegate to call */
		FExtensionHandlerDelegate Delegate;

		/** Has this been removed, if so do not call */
		bool bRemoved;

		/** Construct from an existing delegate, there is no default constructor */
		FExtensionHandlerRegisteredDelegate(FExtensionHandlerDelegate&& InDelegate)
			: Delegate(InDelegate)
			, bRemoved(false)
		{
		}

		/** Call Delegate if it is valid */
		void Execute(AActor* OwningActor, FName EventName)
		{
			if (!bRemoved)
			{
				Delegate.Execute(OwningActor, EventName);
			}
		}
	};

	/** Using a fake multicast delegate so order can be kept consistent */
	using FExtensionHandlerEvent = TMap<FDelegateHandle, TSharedRef<FExtensionHandlerRegisteredDelegate>>;

	/** All active component requests. Used to avoid adding the same component twice if requested from multiple sources */
	TMap<FComponentRequest, int32> RequestTrackingMap;

	/** A map of component classes to instances of that component class made by this component manager */
	TMap<UClass*, TSet<FObjectKey>> ComponentClassToComponentInstanceMap;

	/** A map of actor classes to component classes & addition flags that should be made for that class. */
	TMap<FComponentRequestReceiverClassPath, TSet<FComponentRequestInfo>> ReceiverClassToComponentClassMap;

	/** A map of actor classes to delegate handlers that should be executed for actors of that class. */
	TMap<FComponentRequestReceiverClassPath, FExtensionHandlerEvent> ReceiverClassToEventMap;

#if WITH_EDITORONLY_DATA
	/** Editor-only set to validate that component requests are only being added for actors that call AddReceiver and RemoveReceiver */
	TSet<FObjectKey> AllReceivers;
#endif // WITH_EDITORONLY_DATA

	friend struct FComponentRequestHandle;


	//////////////////////////////////////////////////////////////////////////////////////////////
	// The init state system can be used by components to coordinate their initialization using game-specific states specified as gameplay tags
	// IGameFrameworkInitStateInterface provides a simple implementation that can be inherted by components

public:

	/** Adds a new global actor feature state, either before or after an existing one. This will generally be called from game global or game feature initialization */
	UE_API void RegisterInitState(FGameplayTag NewState, bool bAddBefore, FGameplayTag ExistingState);

	/** Returns true if FeatureState comes after the second state (or is equal) */
	UE_API bool IsInitStateAfterOrEqual(FGameplayTag FeatureState, FGameplayTag RelativeState) const;

	/** Returns the earliest state found for the given feature */
	UE_API FGameplayTag GetInitStateForFeature(AActor* Actor, FName FeatureName) const;

	/** Returns true if feature has reached query state or later */
	UE_API bool HasFeatureReachedInitState(AActor* Actor, FName FeatureName, FGameplayTag FeatureState) const;

	/** Returns the object implementing specified feature, filtered by required state if not none */
	UE_API UObject* GetImplementerForFeature(AActor* Actor, FName FeatureName, FGameplayTag RequiredState = FGameplayTag()) const;

	/** Gets all implementing objects for an actor that are at RequiredState or later, other than excluding feature if specified */
	UE_API void GetAllFeatureImplementers(TArray<UObject*>& OutImplementers, AActor* Actor, FGameplayTag RequiredState, FName ExcludingFeature = NAME_None) const;

	/** Checks to see if all features of object, other than the excluding feature if specified, have reached a specified state or later */
	UE_API bool HaveAllFeaturesReachedInitState(AActor* Actor, FGameplayTag RequiredState, FName ExcludingFeature = NAME_None) const;

	/** Changes the current actor feature state, this will call registered callbacks and return true if anything changed */
	UE_API bool ChangeFeatureInitState(AActor* Actor, FName FeatureName, UObject* Implementer, FGameplayTag FeatureState);

	/** Registers an implementer for a given feature, this will create a feature if required and set the implementer object but will not change the current state */
	UE_API bool RegisterFeatureImplementer(AActor* Actor, FName FeatureName, UObject* Implementer);

	/** Removes an actor and all of it's state information */
	UE_API void RemoveActorFeatureData(AActor* Actor);

	/** Removes an implementing object and any feature states it implements */
	UE_API void RemoveFeatureImplementer(AActor* Actor, UObject* Implementer);
	
	/** 
	 * Registers native delegate for feature state change notifications on a specific actor and may call it immediately
	 * 
	 * @param Actor				The actor to listen for state changes to, if you don't have a specific actor call the Class version instead
	 * @param FeatureName		If not empty, only listen to state changes for the specified feature
	 * @param RequiredState		If specified, only activate if the init state of the feature is equal to or later than this
	 * @param Delegate			Native delegate to call
	 * @param bCallImmediately	If true and the actor feature is already in the specified state, call delegate immediately after registering 
	 * @return DelegateHandle used for later removal
	 */
	UE_API FDelegateHandle RegisterAndCallForActorInitState(AActor* Actor, FName FeatureName, FGameplayTag RequiredState, FActorInitStateChangedDelegate Delegate, bool bCallImmediately = true);

	/** Removes a registered delegate bound to a specific actor */
	UE_API bool UnregisterActorInitStateDelegate(AActor* Actor, FDelegateHandle& Handle);
	
	/**
	 * Registers blueprint delegate for feature state change notifications on a specific actor and may call it immediately
	 *
	 * @param Actor				The actor to listen for state changes to, if you don't have a specific actor call the Class version instead
	 * @param FeatureName		If not empty, only listen to state changes for the specified feature
	 * @param RequiredState		If specified, only activate if the init state of the feature is equal to or later than this
	 * @param Delegate			Native delegate to call
	 * @param bCallImmediately	If true and the actor feature is already in the specified state, call delegate immediately after registering
	 * @return true if delegate was registered
	 */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	UE_API bool RegisterAndCallForActorInitState(AActor* Actor, FName FeatureName, FGameplayTag RequiredState, FActorInitStateChangedBPDelegate Delegate, bool bCallImmediately = true);

	/** Removes a registered delegate bound to a specific actor */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	UE_API bool UnregisterActorInitStateDelegate(AActor* Actor, FActorInitStateChangedBPDelegate DelegateToRemove);


	/**
	 * Registers native delegate for feature state change notifications on a class of actors and may call it immediately
	 *
	 * @param ActorClass		Name of an actor class to listen for changes to
	 * @param FeatureName		If not empty, only listen to state changes for the specified feature
	 * @param RequiredState		If specified, only activate if the init state of the feature is equal to or later than this
	 * @param Delegate			Native delegate to call
	 * @param bCallImmediately	If true and the actor feature is already in the specified state, call delegate immediately after registering
	 * @return DelegateHandle used for later removal
	 */
	UE_API FDelegateHandle RegisterAndCallForClassInitState(const TSoftClassPtr<AActor>& ActorClass, FName FeatureName, FGameplayTag RequiredState, FActorInitStateChangedDelegate Delegate, bool bCallImmediately = true);

	/** Removes a registered delegate bound to a class */
	UE_API bool UnregisterClassInitStateDelegate(const TSoftClassPtr<AActor>& ActorClass, FDelegateHandle& Handle);

	/**
	 * Registers blueprint delegate for feature state change notifications on a class of actors and may call it immediately
	 *
	 * @param ActorClass		Name of an actor class to listen for changes to
	 * @param FeatureName		If not empty, only listen to state changes for the specified feature
	 * @param RequiredState		If specified, only activate if the init state of the feature is equal to or later than this
	 * @param Delegate			Native delegate to call
	 * @param bCallImmediately	If true and the actor feature is already in the specified state, call delegate immediately after registering
	 * @return true if delegate was registered
	 */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	UE_API bool RegisterAndCallForClassInitState(TSoftClassPtr<AActor> ActorClass, FName FeatureName, FGameplayTag RequiredState, FActorInitStateChangedBPDelegate Delegate, bool bCallImmediately = true);

	/** Removes a registered delegate bound to a class */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	UE_API bool UnregisterClassInitStateDelegate(TSoftClassPtr<AActor> ActorClass, FActorInitStateChangedBPDelegate DelegateToRemove);


private:
	/** List of all registered feature states in order */
	TArray<FGameplayTag> InitStateOrder;

	/** State for a specific object implementing an actor feature, should this be in a map instead of an array? */
	struct FActorFeatureState
	{
		FActorFeatureState(FName InFeatureName) : FeatureName(InFeatureName) {}

		/** The feature this is tracking */
		FName FeatureName;

		/** The state when it was last registered */
		FGameplayTag CurrentState;

		/** The object implementing this feature, this can be null */
		TWeakObjectPtr<UObject> Implementer;
	};

	/** Holds the list of feature delegates */
	struct FActorFeatureRegisteredDelegate
	{
		/** Construct from a native or BP Delegate */
		FActorFeatureRegisteredDelegate(FActorInitStateChangedDelegate&& InDelegate, FName InFeatureName = NAME_None, FGameplayTag InInitState = FGameplayTag());
		FActorFeatureRegisteredDelegate(FActorInitStateChangedBPDelegate&& InDelegate, FName InFeatureName = NAME_None, FGameplayTag InInitState = FGameplayTag());

		/** Call the appropriate native/bp delegate, this could invalidate this struct */
		void Execute(AActor* OwningActor, FName FeatureName, UObject* Implementer, FGameplayTag FeatureState);

		/** Check whether this delegate has a valid registration */
		bool IsBound() const
		{
			return Delegate.IsBound() || BPDelegate.IsBound();
		}

		/** Delegate that is called on notification */
		FActorInitStateChangedDelegate Delegate;

		/** BP delegate that is called on notification */
		FActorInitStateChangedBPDelegate BPDelegate;

		/** A handle assigned to this delegate so it acts like a multicast delegate for removal */
		FDelegateHandle DelegateHandle;

		/** If this is not null, will only activate for specific feature names */
		FName RequiredFeatureName;

		/** If this is not null, will only activate for states >= to this */
		FGameplayTag RequiredInitState;

		/** Indicates this delegate has been removed and will soon be destroyed, do not execute */
		bool bRemoved;
	};

	/** Always stored as shared refs to avoid being destroyed during execution */
	using FActorFeatureDelegateList = TArray<TSharedRef<FActorFeatureRegisteredDelegate>>;

	/** Information for each registered actor */
	struct FActorFeatureData
	{
		/** Actor class for cross referencing with the class callbacks */
		TWeakObjectPtr<UClass> ActorClass;

		/** All active features */
		TArray<FActorFeatureState> RegisteredStates;

		/** All delegates bound to this actor */
		FActorFeatureDelegateList RegisteredDelegates;

		/** Properly clear delegate list */
		~FActorFeatureData();
	};

	/** Actors that were registered as tracking feature state */
	TMap<FObjectKey, FActorFeatureData> ActorFeatureMap;

	/** Global delegates for any class name */
	TMap<FComponentRequestReceiverClassPath, FActorFeatureDelegateList> ClassFeatureChangeDelegates;

	/** A queue of state changes to call delegates for, we don't want recursive callbacks */
	TArray<TPair<AActor*, FActorFeatureState> > StateChangeQueue;

	/** Position in state change queue, INDEX_NONE means not actively handling */
	int32 CurrentStateChange;

	/** Find an appropriate state struct if it exists */
	UE_API const FActorFeatureState* FindFeatureStateStruct(const FActorFeatureData* ActorStruct, FName FeatureName, FGameplayTag RequiredState = FGameplayTag()) const;

	/** Add to queue for delegate processing */
	UE_API void ProcessFeatureStateChange(AActor* Actor, const FActorFeatureState* StateChange);

	/** Call all delegates for a specific actor feature change */
	UE_API void CallFeatureStateDelegates(AActor* Actor, FActorFeatureState StateChange);

	/** Call the specified delegate for all matching features on the actor, this should be passed a copy of the original delegate */
	UE_API void CallDelegateForMatchingFeatures(AActor* Actor, FActorFeatureRegisteredDelegate& RegisteredDelegate);

	/** Call the specified delegate for all matching actors and features, this should be passed a copy of the original delegate */
	UE_API void CallDelegateForMatchingActors(UClass* ActorClass, FActorFeatureRegisteredDelegate& RegisteredDelegate);

	/** Gets or creates the actor struct */
	UE_API FActorFeatureData& FindOrAddActorData(AActor* Actor);

	/** Try to removes delegate from list, returns true if it removed anything and clear handle if required */
	UE_API bool RemoveActorFeatureDelegateFromList(FActorFeatureDelegateList& DelegateList, FDelegateHandle& SearchHandle) const;
	UE_API bool RemoveActorFeatureDelegateFromList(FActorFeatureDelegateList& DelegateList, FActorInitStateChangedBPDelegate SearchDelegate) const;

	/** Unregister all class feature change delegates in the relevant packages */
	UE_API void PruneUnboundDelegates(TConstArrayView<UPackage*> Packages);
};

#undef UE_API
