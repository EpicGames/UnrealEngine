// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "UObject/Interface.h"
#include "GameFrameworkInitStateInterface.generated.h"

#define UE_API MODULARGAMEPLAY_API

class FActorInitStateChangedBPDelegate;
struct FActorInitStateChangedParams;

class UGameFrameworkComponentManager;

/** Interface that can be implemented by actors/components to make interacting with the init state system easier */
UINTERFACE(MinimalAPI, NotBlueprintable)
class UGameFrameworkInitStateInterface : public UInterface
{
	GENERATED_BODY()
};

class IGameFrameworkInitStateInterface
{
	GENERATED_BODY()
public:

	/** Returns the Actor this object is bound to, might be this object */
	UE_API virtual AActor* GetOwningActor() const;

	/** Gets the component manager corresponding to this object based on owning actor */
	UE_API UGameFrameworkComponentManager* GetComponentManager() const;

	/** Returns the feature this object implements, this interface is only meant for simple objects with a single feature like Actor */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	virtual FName GetFeatureName() const { return NAME_None; }

	/** Returns the current feature state of this object, the default behavior is to query the manager */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	UE_API virtual FGameplayTag GetInitState() const;

	/** Checks the component manager to see if we have already reached the desired state or a later one */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	UE_API virtual bool HasReachedInitState(FGameplayTag DesiredState) const;

	/** Should be overridden to perform class-specific checks to see if the desired state can be reached */
	virtual bool CanChangeInitState(UGameFrameworkComponentManager* Manager, FGameplayTag CurrentState, FGameplayTag DesiredState) const { return true; }

	/** Should be overridden to perform class-specific state changes, this is called right before notifying the component manager */
	virtual void HandleChangeInitState(UGameFrameworkComponentManager* Manager, FGameplayTag CurrentState, FGameplayTag DesiredState) {}

	/** Checks to see if a change is possible, then calls execute and notify */
	UE_API virtual bool TryToChangeInitState(FGameplayTag DesiredState);

	/** Tries to follow a chain of connected init states, will progress states in order and returns the final state reached */
	UE_API virtual FGameplayTag ContinueInitStateChain(const TArray<FGameplayTag>& InitStateChain);

	/** Override to try and progress the default initialization path, likely using ContinueInitStateChain */
	virtual void CheckDefaultInitialization() {}

	/** This will call CheckDefaultInitialization on all other feature implementers using this interface, useful to update the state of any dependencies */
	UE_API virtual void CheckDefaultInitializationForImplementers();

	/** Signature for handling a game feature state, this is not registered by default */
	virtual void OnActorInitStateChanged(const FActorInitStateChangedParams& Params) {}

	/** Call to bind the OnActorInitStateChanged function to the appropriate delegate on the component manager */
	UE_API virtual void BindOnActorInitStateChanged(FName FeatureName, FGameplayTag RequiredState, bool bCallIfReached);

	/** Call to register with the component manager during spawn if this is a game world */
	UE_API virtual void RegisterInitStateFeature();

	/** Unregisters state and delegate binding with component manager */
	UE_API virtual void UnregisterInitStateFeature();

	/** Binds a BP delegate to get called on a state change for this feature */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	UE_API virtual bool RegisterAndCallForInitStateChange(FGameplayTag RequiredState, FActorInitStateChangedBPDelegate Delegate, bool bCallImmediately = true);

	/** Unbinds a BP delegate from changes to this feature */
	UFUNCTION(BlueprintCallable, Category = "InitState")
	UE_API virtual bool UnregisterInitStateDelegate(FActorInitStateChangedBPDelegate Delegate);

	/** Returns Current state and any additional debug information for the active state */
	UE_API virtual FString GetDebugState() const;

protected:
	/** Default handle created from calling BindOnActorInitStateChanged */
	FDelegateHandle ActorInitStateChangedHandle;
};

#undef UE_API
