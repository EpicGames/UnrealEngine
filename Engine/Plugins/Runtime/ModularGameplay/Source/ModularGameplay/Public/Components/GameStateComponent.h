// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/GameStateBase.h"
#include "GameFrameworkComponent.h"
#include "Engine/World.h"
#include "GameStateComponent.generated.h"

#define UE_API MODULARGAMEPLAY_API

/**
 * GameStateComponent is an actor component made for AGameStateBase and receives GameState events.
 */
UCLASS(MinimalAPI)
class UGameStateComponent : public UGameFrameworkComponent
{
	GENERATED_BODY()

public:
	
	UE_API UGameStateComponent(const FObjectInitializer& ObjectInitializer);

	/** Gets the game state that owns the component, this will always be valid during gameplay but can return null in the editor */
	template <class T>
	T* GetGameState() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, AGameStateBase>::Value, "'T' template parameter to GetGameState must be derived from AGameStateBase");
		return Cast<T>(GetOwner());
	}

	template <class T>
	T* GetGameStateChecked() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, AGameStateBase>::Value, "'T' template parameter to GetGameStateChecked must be derived from AGameStateBase");
		return CastChecked<T>(GetOwner());
	}

	//////////////////////////////////////////////////////////////////////////////
	// GameState accessors, only valid if called during gameplay
	//////////////////////////////////////////////////////////////////////////////

	/** Gets the game mode that owns this component, this will always return null on the client */
	template <class T>
	T* GetGameMode() const
	{
		// Note: Intentionally getting the game mode from the world instead of the game state as it can be null during game state initialization
		static_assert(TPointerIsConvertibleFromTo<T, AGameModeBase>::Value, "'T' template parameter to GetGameMode must be derived from AGameModeBase");
		const UWorld* World = GetWorld();
		return World ? World->GetAuthGameMode<T>() : nullptr;
	}

public:

	//////////////////////////////////////////////////////////////////////////////
	// GameState events
	//////////////////////////////////////////////////////////////////////////////

	/** Called when gameplay has fully started */
	virtual void HandleMatchHasStarted() {}

	/** Called when gameplay has fully ended */
	virtual void HandleMatchHasEnded() {}
};

#undef UE_API
