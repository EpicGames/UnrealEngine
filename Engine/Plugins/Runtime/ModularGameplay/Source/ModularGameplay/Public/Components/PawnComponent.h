// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFrameworkComponent.h"
#include "GameFramework/Pawn.h" // IWYU pragma: keep
#include "PawnComponent.generated.h"

#define UE_API MODULARGAMEPLAY_API

class APlayerState;

/**
 * PawnComponent is an actor component made for APawn and receives pawn events.
 */
UCLASS(MinimalAPI)
class UPawnComponent : public UGameFrameworkComponent
{
	GENERATED_BODY()

public:
	
	UE_API UPawnComponent(const FObjectInitializer& ObjectInitializer);

	/** Gets the pawn that owns the component, this will always be valid during gameplay but can return null in the editor */
	template <class T>
	T* GetPawn() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APawn>::Value, "'T' template parameter to GetPawn must be derived from APawn");
		return Cast<T>(GetOwner());
	}

	template <class T>
	T* GetPawnChecked() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APawn>::Value, "'T' template parameter to GetPawnChecked must be derived from APawn");
		return CastChecked<T>(GetOwner());
	}

	//////////////////////////////////////////////////////////////////////////////
	// Pawn accessors, only valid if called during gameplay
	//////////////////////////////////////////////////////////////////////////////

	/** Gets the player state that owns the component, this can return null on clients for player pawns that are still being replicated */
	template <class T>
	T* GetPlayerState() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APlayerState>::Value, "'T' template parameter to GetPlayerState must be derived from APlayerState");
		return GetPawnChecked<APawn>()->GetPlayerState<T>();
	}

	/** Gets the controller that owns the component, this will usually be null on clients */
	template <class T>
	T* GetController() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, AController>::Value, "'T' template parameter to GetController must be derived from AController");
		return GetPawnChecked<APawn>()->GetController<T>();
	}

};

#undef UE_API
