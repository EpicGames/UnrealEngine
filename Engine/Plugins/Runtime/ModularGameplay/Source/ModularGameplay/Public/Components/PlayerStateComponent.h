// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFrameworkComponent.h"
#include "GameFramework/PlayerState.h" // IWYU pragma: keep
#include "PlayerStateComponent.generated.h"

#define UE_API MODULARGAMEPLAY_API

class APlayerState;

/**
 * PlayerStateComponent is an actor component made for APlayerState and receives PlayerState events.
 */
UCLASS(MinimalAPI)
class UPlayerStateComponent : public UGameFrameworkComponent
{
	GENERATED_BODY()

public:
	
	UE_API UPlayerStateComponent(const FObjectInitializer& ObjectInitializer);

	/** Gets the player state that owns the component, this will always be valid during gameplay but can return null in the editor */
	template <class T>
	T* GetPlayerState() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APlayerState>::Value, "'T' template parameter to GetPlayerState must be derived from APlayerState");
		return Cast<T>(GetOwner());
	}

	template <class T>
	T* GetPlayerStateChecked() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APlayerState>::Value, "'T' template parameter to GetPlayerStateChecked must be derived from APlayerState");
		return CastChecked<T>(GetOwner());
	}

	//////////////////////////////////////////////////////////////////////////////
	// PlayerState accessors, only valid if called during gameplay
	//////////////////////////////////////////////////////////////////////////////

	/** Gets the pawn that owns the component, this will return null if the pawn does not exist or is still replicating */
	template <class T>
	T* GetPawn() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APawn>::Value, "'T' template parameter to GetPawn must be derived from APawn");
		return GetPlayerStateChecked<APlayerState>()->GetPawn<T>();
	}

public:

	//////////////////////////////////////////////////////////////////////////////
	// PlayerState events
	//////////////////////////////////////////////////////////////////////////////

	/** Reset properties to default state */
	virtual void Reset() {}

	/** Copy properties which need to be saved in inactive PlayerState */
	virtual void CopyProperties(UPlayerStateComponent* TargetPlayerStateComponent) {}

};

#undef UE_API
