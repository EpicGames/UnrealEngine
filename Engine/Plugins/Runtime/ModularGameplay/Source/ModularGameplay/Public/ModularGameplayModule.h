// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class IModularGameplayModule : public IModuleInterface
{
public:
	static inline IModularGameplayModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IModularGameplayModule>("ModularGameplay");
	}
};
